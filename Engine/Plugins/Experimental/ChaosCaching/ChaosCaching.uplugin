{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "ChaosCaching",
	"Description": "Chaos Cache asset support for recording and playing back physics simulations",
	"Category": "Physics",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "ChaosCaching",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "ChaosCachingEditor",
			"Type": "Editor",
			"LoadingPhase": "PreDefault"
		}
	],
	"Plugins": [
		{
			"Name": "Takes",
			"Enabled": true
		}
	]
}