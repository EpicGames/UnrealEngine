// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ChaosCaching : ModuleRules
	{
        public ChaosCaching(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
                    "Engine",
                    "EngineSettings",
                    "RenderCore",
                    "RHI",
					"GeometryCollectionEngine",
					"ChaosSolverEngine",
					"MovieScene",
					"MovieSceneTracks",
					"LevelSequence",
				});

			if(Target.bBuildEditor)
            {
				// Slate/Editor extensions
				PublicDependencyModuleNames.AddRange(
					new string[]
					{
						"UnrealEd",
						"SlateCore",
						"Slate"
					});
            }

            SetupModulePhysicsSupport(Target);
			PrivateDefinitions.Add("CHAOS_INCLUDE_LEVEL_1=1");
		}
	}
}
