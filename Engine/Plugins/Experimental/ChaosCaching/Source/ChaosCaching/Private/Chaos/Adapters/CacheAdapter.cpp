// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Adapters/CacheAdapter.h"
#include "Chaos/CacheManagerActor.h"
#include "Features/IModularFeatures.h"
#include "Components/PrimitiveComponent.h"

// Feature registration name for modular features
const FName Chaos::FComponentCacheAdapter::FeatureName("ChaosCacheAdapter");

// Engine adapters will always use priorities between EngineAdapterPriorityBegin and UserAdapterPriorityBegin
const uint8 Chaos::FComponentCacheAdapter::EngineAdapterPriorityBegin(0);

// Any priority of this or above is a user implemented adapter
const uint8 Chaos::FComponentCacheAdapter::UserAdapterPriorityBegin(1 << 3);

DEFINE_LOG_CATEGORY(LogCacheAdapter);

namespace Chaos
{
	bool FComponentCacheAdapter::InitializeForRecord(UPrimitiveComponent* InComponent, FObservedComponent& InObserved)
	{
		return InitializeForRecord(InComponent, InObserved.Cache);
	}

	bool FComponentCacheAdapter::InitializeForPlayback(UPrimitiveComponent* InComponent, FObservedComponent& InObserved, float InTime)
	{
		return InitializeForPlayback(InComponent, InObserved.Cache, InTime);
	}

	void RegisterAdapter(FComponentCacheAdapter* InAdapter)
	{
		check(InAdapter);

		IModularFeatures& ModularFeatures = IModularFeatures::Get();

		TArray<FComponentCacheAdapter*> Adapters =
			ModularFeatures.GetModularFeatureImplementations<FComponentCacheAdapter>(
				FComponentCacheAdapter::FeatureName);

		// Multiple adapters can't directly support the same class with the same priority.
		// If this happens either there is a collision or an adapter has been registered twice
		UClass*                             NewClass         = InAdapter->GetDesiredClass();
		FComponentCacheAdapter::SupportType NewSupportType   = InAdapter->SupportsComponentClass(NewClass);
		uint8                               NewPriority      = InAdapter->GetPriority();
		bool                                bNewAdapterValid = true;

		for(FComponentCacheAdapter* Adapter : Adapters)
		{
			if(NewClass == Adapter->GetDesiredClass() && NewSupportType == Adapter->SupportsComponentClass(NewClass)
			   && NewPriority == Adapter->GetPriority())
			{
				UE_LOGF(LogCacheAdapter,
					   Error,
					   "Attempted to register a cache adapter with GUID %ls however the desired class %ls is "
							"already registered at the requested priority (%u) by a cache adapter with GUID %ls",
					   *InAdapter->GetGuid().ToString(),
					   *NewClass->GetName(),
					   NewPriority,
					   *Adapter->GetGuid().ToString());

				bNewAdapterValid = false;
				break;
			}
		}

		if(bNewAdapterValid)
		{
			ModularFeatures.RegisterModularFeature("ChaosCacheAdapter", InAdapter);
		}
	}

	void UnregisterAdapter(FComponentCacheAdapter* InAdapter)
	{
		check(InAdapter);

		IModularFeatures& ModularFeatures = IModularFeatures::Get();

		// Directly calling unregister on the modular features will trigger an unregister broadcast
		// even if the feature wasn't actually registered, so verify here first to avoid over-broadcasting
		TArray<FComponentCacheAdapter*> Adapters =
			ModularFeatures.GetModularFeatureImplementations<FComponentCacheAdapter>(
				FComponentCacheAdapter::FeatureName);

		if(Adapters.Contains(InAdapter))
		{
			ModularFeatures.UnregisterModularFeature(FComponentCacheAdapter::FeatureName, InAdapter);
		}
	}

	FComponentCacheAdapter* FAdapterUtil::GetBestAdapterForClass(TSubclassOf<UPrimitiveComponent> InComponentClass, bool bAllowDerived)
	{
		// Build list of adapters for our observed components
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		TArray<FComponentCacheAdapter*> Adapters = ModularFeatures.GetModularFeatureImplementations<FComponentCacheAdapter>(FComponentCacheAdapter::FeatureName);

		auto ByPriority = [](const FComponentCacheAdapter* A, const FComponentCacheAdapter* B)
		{
			return A->GetPriority() < B->GetPriority();
		};

		UClass* ActualClass = InComponentClass.Get();
		TArray<FComponentCacheAdapter*> DirectAdapters = Adapters.FilterByPredicate([ActualClass](const FComponentCacheAdapter* InTest)
		{
			return InTest && InTest->SupportsComponentClass(ActualClass) == Chaos::FComponentCacheAdapter::SupportType::Direct;
		});

		TArray<FComponentCacheAdapter*> DerivedAdapters = Adapters.FilterByPredicate([ActualClass](const FComponentCacheAdapter* InTest)
		{
			return InTest && InTest->SupportsComponentClass(ActualClass) == Chaos::FComponentCacheAdapter::SupportType::Derived;
		});

		Algo::Sort(DirectAdapters, ByPriority);
		Algo::Sort(DerivedAdapters, ByPriority);

		if(DirectAdapters.Num() > 0)
		{
			return DirectAdapters[0];
		}

		if(bAllowDerived && DerivedAdapters.Num() > 0)
		{
			return DerivedAdapters[0];
		}

		return nullptr;
	}

}    // namespace Chaos
