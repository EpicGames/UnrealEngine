// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Adapters/GeometryCollectionComponentCacheAdapter.h"
#include "Chaos/Adapters/CacheAdapter.h"
#include "Chaos/ChaosCache.h"
#include "Engine/World.h"
#include "EventManager.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "Physics/Experimental/ChaosEventRelay.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"
#include "Chaos/ChaosSolverActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionComponentCacheAdapter)

FName FEnableStateEvent::EventName("GC_Enable");
FName FBreakingEvent::EventName("GC_Breaking");
FName FCollisionEvent::EventName("GC_Collision");
FName FTrailingEvent::EventName("GC_Trailing");

namespace Chaos
{

	namespace GeometryCollectionCacheAdapter::CVars
	{
		bool bUseKinematicTarget = true;
		FAutoConsoleVariableRef CVARUseKinematicTarget(TEXT("p.Chaos.Cache.GeometryCollectionPlayUseKinematicTarget"), bUseKinematicTarget,
			TEXT("When in Play cache mode, use kinematic target to move the geometry collection rigid body instead of setting their position directly [def:true]"));
	}

	FComponentCacheAdapter::SupportType FGeometryCollectionCacheAdapter::SupportsComponentClass(UClass* InComponentClass) const
	{
		UClass* Desired = GetDesiredClass();
		if(InComponentClass == Desired)
		{
			return FComponentCacheAdapter::SupportType::Direct;
		}
		else if(InComponentClass->IsChildOf(Desired))
		{
			return FComponentCacheAdapter::SupportType::Derived;
		}

		return FComponentCacheAdapter::SupportType::None;
	}

	UClass* FGeometryCollectionCacheAdapter::GetDesiredClass() const
	{
		return UGeometryCollectionComponent::StaticClass();
	}

	uint8 FGeometryCollectionCacheAdapter::GetPriority() const
	{
		return EngineAdapterPriorityBegin;
	}
	
	void FGeometryCollectionCacheAdapter::Record_PostSolve(UPrimitiveComponent* InComp, const FTransform& InRootTransform, FPendingFrameWrite& OutFrame, Chaos::FReal InTime) const
	{

		EnsureIsInPhysicsThreadContext();


		using FClusterParticle = Chaos::FPBDRigidClusteredParticleHandle;
		using FRigidParticle = Chaos::FPBDRigidParticleHandle;

		UGeometryCollectionComponent*    Comp  = CastChecked<UGeometryCollectionComponent>(InComp);
		FGeometryCollectionPhysicsProxy* Proxy = Comp->GetPhysicsProxy();
		
		if(!Proxy)
		{
			return;
		}

		if (!CachedData.Contains(Proxy))
		{
			return;
		}

		const FTransform WorldToComponent = Proxy->GetSimParameters().WorldTransform.Inverse();

		const FCachedEventData& ProxyCachedEventData = CachedData[Proxy];

		const Chaos::FPhysicsSolver* Solver = Proxy->GetSolver<Chaos::FPhysicsSolver>();
		TSharedPtr<const FGeometryCollection> RestCollection = Proxy->GetSimParameters().RestCollectionShared;

		if(!RestCollection || !Solver)
		{
			return;
		}

		const FPhysScene* Scene = static_cast<FPhysScene*>(Solver->PhysSceneHack);
		if (!Scene)
		{
			return;
		}

		FGeometryDynamicCollection&            Collection       = Proxy->GetPhysicsCollection();
		const TManagedArray<FTransform>&       MassToLocal      = RestCollection->GetAttribute<FTransform>("MassToLocal", FGeometryCollection::TransformGroup);
		const TManagedArray<int32>&            Parents			= RestCollection->Parent;
		const TManagedArray<TSet<int32>>&	   Children         = RestCollection->Children;
		const TArray<FBreakingData>&		   Breaks           = Solver->GetEvolution()->GetRigidClustering().GetAllClusterBreakings();
		const FTransform					   WorldToActor		= InRootTransform.Inverse();

		// A transform index exists for each 'real' (i.e. leaf node in the rest collection)
		const int32 NumTransforms = Collection.NumElements(FGeometryCollection::TransformGroup);

		// Pre-alloc once for worst case.
		OutFrame.PendingParticleData.Reserve(NumTransforms);

		// Record the cluster indices that released this frame. Those particles become
		// inactive: to make random access reads convenient, we set their transforms to identity.
		TSet<int32> ReleasedClusters;
		ReleasedClusters.Reserve(Breaks.Num());
		
		TArray<int32> RelatedBreaks;
		RelatedBreaks.Reserve(Breaks.Num());
		
		for(const FBreakingData& Break : Breaks)
		{
			// Is the proxy pending destruction? If they are no longer tracked by the PhysScene, the proxy is deleted or pending deletion.
			if (Scene->GetOwningComponent<UPrimitiveComponent>(Break.Proxy) == nullptr)
			{
				continue;
			}
			
			if(Break.Proxy->GetType() == EPhysicsProxyType::GeometryCollectionType)
			{
				FGeometryCollectionPhysicsProxy* ConcreteProxy = static_cast<FGeometryCollectionPhysicsProxy*>(Break.Proxy);

				if(ConcreteProxy == Proxy)
				{
					// Record the cluster it belonged to.
					ReleasedClusters.Add(Parents[Break.TransformGroupIndex]);

					// Force a break on all children of the cluster
					for (int32 ChildToBreak : Children[Parents[Break.TransformGroupIndex]])
					{
						RelatedBreaks.AddUnique(ChildToBreak);
					}
				}
			}
		}

		for(int32 TransformIndex = 0; TransformIndex < NumTransforms; ++TransformIndex)
		{
			FClusterParticle* Handle = Proxy->GetParticle_Internal(TransformIndex);
			
			if(Handle)
			{
				const FRigidParticle* Parent = Handle ? Handle->ClusterIds().Id : nullptr;
				const FClusterParticle* ParentAsCluster = Parent ? Parent->CastToClustered() : nullptr;
				const bool bParentIsInternalCluster = ParentAsCluster ? ParentAsCluster->InternalCluster() : false;
				const bool bParentIsActiveInternalCluster = bParentIsInternalCluster && !Parent->Disabled();

				const bool bParticleDisabled = Handle->Disabled();
				if(!bParticleDisabled || bParentIsActiveInternalCluster)
				{
					OutFrame.PendingParticleData.AddDefaulted();
					FPendingParticleWrite& Pending = OutFrame.PendingParticleData.Last();

					Pending.ParticleIndex = TransformIndex;
					
					// All recorded transforms are in actor space, ie relative to the Cache Manager making the recording.
					FTransform LocalTransform = MassToLocal[TransformIndex].Inverse() * FTransform(Handle->GetR(), Handle->GetX());
					FTransform ActorSpaceTransform = LocalTransform * WorldToActor;
					Pending.PendingTransform = ActorSpaceTransform;
				}

				if (RelatedBreaks.Contains(TransformIndex))
				{
					OutFrame.PushEvent(FEnableStateEvent::EventName, InTime, FEnableStateEvent(TransformIndex, true));
				}
			}
		}

		// If a cluster particle released, set its transform to identity.
		for (int32 TransformIndex : ReleasedClusters)
		{
			if (TransformIndex > INDEX_NONE)
			{
				FClusterParticle* Handle = Proxy->GetParticle_Internal(TransformIndex);
				if (Handle)
				{
					OutFrame.PendingParticleData.AddDefaulted();
					FPendingParticleWrite& Pending = OutFrame.PendingParticleData.Last();
					Pending.ParticleIndex = TransformIndex;
					FTransform LocalTransform = MassToLocal[TransformIndex].Inverse() * FTransform(Handle->GetR(), Handle->GetX());
					FTransform ActorSpaceTransform = LocalTransform * WorldToActor;
					Pending.PendingTransform = ActorSpaceTransform;
					Pending.bPendingDeactivate = true;
				}
			}
		}

		if ((Proxy->GetSimParameters().bGenerateBreakingData || Proxy->GetSimParameters().bGenerateGlobalBreakingData) && BreakingDataArray && ProxyCachedEventData.ProxyBreakingDataIndices)
		{
			for (int32 Index : *ProxyCachedEventData.ProxyBreakingDataIndices)
			{
				if (BreakingDataArray->IsValidIndex(Index))
				{
					const FBreakingData& BreakingData = (*BreakingDataArray)[Index];
					if (BreakingData.TransformGroupIndex > INDEX_NONE)
					{
						OutFrame.PushEvent(FBreakingEvent::EventName, InTime, FBreakingEvent(BreakingData.TransformGroupIndex, BreakingData, WorldToComponent));
					}
				}
				
			}
		}
		
		if ((Proxy->GetSimParameters().bGenerateCollisionData || Proxy->GetSimParameters().bGenerateGlobalCollisionData) && CollisionDataArray && ProxyCachedEventData.ProxyCollisionDataIndices)
		{
			for (int32 Index : *ProxyCachedEventData.ProxyCollisionDataIndices)
			{
				if (CollisionDataArray->IsValidIndex(Index))
				{
					const FCollidingData& CollisionData = (*CollisionDataArray)[Index];
					OutFrame.PushEvent(FCollisionEvent::EventName, InTime, FCollisionEvent(CollisionData, WorldToComponent));
				}

			}
		}
		
		if ((Proxy->GetSimParameters().bGenerateTrailingData || Proxy->GetSimParameters().bGenerateGlobalTrailingData) && TrailingDataArray && ProxyCachedEventData.ProxyTrailingDataIndices)
		{
			for (int32 Index : *ProxyCachedEventData.ProxyTrailingDataIndices)
			{
				if (TrailingDataArray->IsValidIndex(Index))
				{
					const FTrailingData& TrailingData = (*TrailingDataArray)[Index];
					if (TrailingData.TransformGroupIndex > INDEX_NONE)
					{
						OutFrame.PushEvent(FTrailingEvent::EventName, InTime, FTrailingEvent(TrailingData.TransformGroupIndex, TrailingData, WorldToComponent));
					}
				}

			}
		}

		// Never going to change again till freed after writing to the cache so free up the extra space we reserved
		OutFrame.PendingParticleData.Shrink();
	}
	
	void FGeometryCollectionCacheAdapter::Playback_PreSolve(UPrimitiveComponent*							   InComponent,
															UChaosCache*									   InCache,
															Chaos::FReal                                       InTime,
															FPlaybackTickRecord&							   TickRecord,
															TArray<TPBDRigidParticleHandle<Chaos::FReal, 3>*>& OutUpdatedRigids) const
	{
		EnsureIsInPhysicsThreadContext();
		
		using FClusterParticle = Chaos::FPBDRigidClusteredParticleHandle;
		using FRigidParticle = Chaos::FPBDRigidParticleHandle;

		UGeometryCollectionComponent*    Comp  = CastChecked<UGeometryCollectionComponent>(InComponent);
		FGeometryCollectionPhysicsProxy* Proxy = Comp->GetPhysicsProxy();
		
		if(!Proxy)
		{
			return;
		}

		const FTransform ComponentToWorld = Proxy->GetSimParameters().WorldTransform;
		TSharedPtr<const FGeometryCollection> RestCollection = Proxy->GetSimParameters().RestCollectionShared;
		Chaos::FPhysicsSolver* Solver = Proxy->GetSolver<Chaos::FPhysicsSolver>();

		if(!RestCollection || !Solver)
		{
			return;
		}

		FGeometryDynamicCollection&      Collection       = Proxy->GetPhysicsCollection();
		const TManagedArray<FTransform>& MassToLocal      = RestCollection->GetAttribute<FTransform>("MassToLocal", FGeometryCollection::TransformGroup);

		FCacheEvaluationContext Context(TickRecord);
		Context.bEvaluateTransform = true;
		Context.bEvaluateCurves    = false;
		Context.bEvaluateEvents    = true;

		FCacheEvaluationResult EvaluatedResult = InCache->Evaluate(Context, &MassToLocal.GetConstArray());

		const int32                      NumEventTracks = EvaluatedResult.Events.Num();
		const TArray<FCacheEventHandle>* EnableEvents   = EvaluatedResult.Events.Find(FEnableStateEvent::EventName);
		const TArray<FCacheEventHandle>* BreakingEvents = EvaluatedResult.Events.Find(FBreakingEvent::EventName);
		const TArray<FCacheEventHandle>* CollisionEvents = EvaluatedResult.Events.Find(FCollisionEvent::EventName);
		const TArray<FCacheEventHandle>* TrailingEvents = EvaluatedResult.Events.Find(FTrailingEvent::EventName);

		if(EnableEvents)
		{
			TMap<FClusterParticle*, TArray<FRigidParticle*>> NewClusters;
			for(const FCacheEventHandle& Handle : *EnableEvents)
			{
				if( const FEnableStateEvent* Event = Handle.GetConst<FEnableStateEvent>())
				{
					if(Chaos::FPBDRigidClusteredParticleHandle* ChildParticle = Proxy->GetParticle_Internal(Event->Index))
					{
						if (ChildParticle->ObjectState() != EObjectStateType::Kinematic)
						{
							// If a field or other external actor set the particle to static or dynamic we no longer apply the cache
							continue;
						}

						if (FRigidParticle* ClusterParent = ChildParticle->ClusterIds().Id)
						{
							if (FClusterParticle* Parent = ClusterParent->CastToClustered())
							{
								TArray<FRigidParticle*>& Cluster = NewClusters.FindOrAdd(Parent);
								Cluster.Add(ChildParticle);
							}
						}
						else
						{
							// This is a cluster parent
							ChildParticle->SetDisabled(!Event->bEnable);
						}
					}
				}
			}

			for(TPair<FClusterParticle*, TArray<FRigidParticle*>> Cluster : NewClusters)
			{
				TArray<FRigidParticle*>& ChildrenParticles = Cluster.Value;
				if (ChildrenParticles.Num())
				{
					FRigidParticle* ClusterHandle = nullptr;
					
					for (FRigidParticle* ChildHandle : ChildrenParticles)
					{
						if (FClusterParticle* ClusteredChildHandle = ChildHandle->CastToClustered())
						{
							if (ClusteredChildHandle->Disabled() && ClusteredChildHandle->ClusterIds().Id != nullptr)
							{
								if (ensure(!ClusterHandle || ClusteredChildHandle->ClusterIds().Id == ClusterHandle))
								{
									ClusterHandle = ClusteredChildHandle->ClusterIds().Id;
								}
								else
								{
									break; //shouldn't be here
								}
							}
						}
					}
					if (ClusterHandle)
					{
						Solver->GetEvolution()->GetRigidClustering().ReleaseClusterParticlesNoInternalCluster(ClusterHandle->CastToClustered(), true);
					}
				}
			}
		}

		if ((Proxy->GetSimParameters().bGenerateBreakingData || Proxy->GetSimParameters().bGenerateGlobalBreakingData) && BreakingEvents)
		{
			const FSolverBreakingEventFilter* SolverBreakingEventFilter = Solver->GetEventFilters()->GetBreakingFilter();

			for (const FCacheEventHandle& Handle : *BreakingEvents)
			{
				if (const FBreakingEvent* Event = Handle.GetConst<FBreakingEvent>())
				{
					if (Chaos::FPBDRigidClusteredParticleHandle* Particle = Proxy->GetParticle_Internal(Event->Index))
					{
						if (Particle->ObjectState() != EObjectStateType::Kinematic)
						{
							// If a field or other external actor set the particle to static or dynamic we no longer apply the cache
							continue;
						}

						FBreakingData CachedBreak;
						CachedBreak.Proxy = Proxy;
						CachedBreak.Location = ComponentToWorld.TransformPosition(Event->Location);
						CachedBreak.Orientation = ComponentToWorld.TransformRotation(Event->Orientation);
						CachedBreak.Velocity = ComponentToWorld.TransformVector(Event->Velocity);
						CachedBreak.AngularVelocity = Event->AngularVelocity;
						CachedBreak.Mass = Event->Mass;
						CachedBreak.BoundingBox = TAABB<FReal, 3>(Event->BoundingBoxMin, Event->BoundingBoxMax);
						CachedBreak.BoundingBox = CachedBreak.BoundingBox.TransformedAABB(ComponentToWorld);
						CachedBreak.SetEmitterFlag(Proxy->GetSimParameters().bGenerateBreakingData, Proxy->GetSimParameters().bGenerateGlobalBreakingData);

						if (!SolverBreakingEventFilter->Enabled() || SolverBreakingEventFilter->Pass(CachedBreak))
						{
							float TimeStamp = Solver->GetSolverTime();
							Solver->GetEventManager()->AddEvent<FBreakingEventData>(EEventType::Breaking, [&CachedBreak, TimeStamp](FBreakingEventData& BreakingEventData)
								{
									if (BreakingEventData.BreakingData.TimeCreated != TimeStamp)
									{
										BreakingEventData.BreakingData.AllBreakingsArray.Reset();
										BreakingEventData.BreakingData.TimeCreated = TimeStamp;
									}
									BreakingEventData.BreakingData.AllBreakingsArray.Add(CachedBreak);
								});
						}
					}
				}
			}
		}

		if ((Proxy->GetSimParameters().bGenerateTrailingData || Proxy->GetSimParameters().bGenerateGlobalTrailingData) && TrailingEvents)
		{
			const FSolverTrailingEventFilter* SolverTrailingEventFilter = Solver->GetEventFilters()->GetTrailingFilter();

			for (const FCacheEventHandle& Handle : *TrailingEvents)
			{
				if (const FTrailingEvent* Event = Handle.GetConst<FTrailingEvent>())
				{
					if (Chaos::FPBDRigidClusteredParticleHandle* Particle = Proxy->GetParticle_Internal(Event->Index))
					{
						if (Particle->ObjectState() != EObjectStateType::Kinematic)
						{
							// If a field or other external actor set the particle to static or dynamic we no longer apply the cache
							continue;
						}

						FTrailingData CachedTrail;
						CachedTrail.Proxy = Proxy;
						CachedTrail.Location = ComponentToWorld.TransformPosition(Event->Location);
						CachedTrail.Orientation = ComponentToWorld.TransformRotation(Event->Orientation);
						CachedTrail.Velocity = ComponentToWorld.TransformVector(Event->Velocity);
						CachedTrail.AngularVelocity = Event->AngularVelocity;
						CachedTrail.BoundingBox = TAABB<FReal, 3>(Event->BoundingBoxMin, Event->BoundingBoxMax);
						CachedTrail.BoundingBox = CachedTrail.BoundingBox.TransformedAABB(ComponentToWorld);
						CachedTrail.Mass = Particle->M();
						CachedTrail.TransformGroupIndex = INDEX_NONE; // todo(ccaillaud) we should probably record that info in the cache : but we need to be careful that this value may change with the GC
						CachedTrail.SetEmitterFlag(Proxy->GetSimParameters().bGenerateTrailingData, Proxy->GetSimParameters().bGenerateGlobalTrailingData);

						if (!SolverTrailingEventFilter->Enabled() || SolverTrailingEventFilter->Pass(CachedTrail))
						{
							float TimeStamp = Solver->GetSolverTime();
							Solver->GetEventManager()->AddEvent<FTrailingEventData>(EEventType::Trailing, [&CachedTrail , TimeStamp](FTrailingEventData& TrailingEventData)
								{
									if (TrailingEventData.TrailingData.TimeCreated != TimeStamp)
									{
										TrailingEventData.TrailingData.AllTrailingsArray.Reset();
										TrailingEventData.TrailingData.TimeCreated = TimeStamp;
									}
									TrailingEventData.TrailingData.AllTrailingsArray.Add(CachedTrail);
								});
						}
					}
				}
			}
		}

		if ((Proxy->GetSimParameters().bGenerateCollisionData || Proxy->GetSimParameters().bGenerateGlobalCollisionData) && CollisionEvents)
		{

			const FSolverCollisionEventFilter* SolverCollisionEventFilter = Solver->GetEventFilters()->GetCollisionFilter();
			for (const FCacheEventHandle& Handle : *CollisionEvents)
			{
				if (const FCollisionEvent* Event = Handle.GetConst<FCollisionEvent>())
				{
					FCollidingData CachedCollision;
					CachedCollision.Location = ComponentToWorld.TransformPosition(Event->Location);
					CachedCollision.AccumulatedImpulse = ComponentToWorld.TransformVector(Event->AccumulatedImpulse);
					CachedCollision.Normal = ComponentToWorld.TransformVector(Event->Normal);
					CachedCollision.Velocity1 = ComponentToWorld.TransformVector(Event->Velocity1);
					CachedCollision.Velocity2 = ComponentToWorld.TransformVector(Event->Velocity2);
					CachedCollision.DeltaVelocity1 = ComponentToWorld.TransformVector(Event->DeltaVelocity1);
					CachedCollision.DeltaVelocity2 = ComponentToWorld.TransformVector(Event->DeltaVelocity2);
					CachedCollision.AngularVelocity1 = Event->AngularVelocity1;
					CachedCollision.AngularVelocity2 = Event->AngularVelocity2;
					CachedCollision.Mass1 = Event->Mass1;
					CachedCollision.Mass2 = Event->Mass2;
					CachedCollision.PenetrationDepth = Event->PenetrationDepth;
					
					if (!SolverCollisionEventFilter->Enabled() || SolverCollisionEventFilter->Pass(CachedCollision))
					{
						float TimeStamp = Solver->GetSolverTime();
						Solver->GetEventManager()->AddEvent<FCollisionEventData>(EEventType::Collision, [&CachedCollision, TimeStamp, Proxy](FCollisionEventData& CollisionEventData)
							{
								if (CollisionEventData.CollisionData.TimeCreated != TimeStamp)
								{
									CollisionEventData.CollisionData.AllCollisionsArray.Reset();
									CollisionEventData.PhysicsProxyToCollisionIndices.Reset();
									CollisionEventData.CollisionData.TimeCreated = TimeStamp;
								}
								int32 NewIdx = CollisionEventData.CollisionData.AllCollisionsArray.Add(CachedCollision);
								CollisionEventData.PhysicsProxyToCollisionIndices.PhysicsProxyToIndicesMap.FindOrAdd(Proxy).Add(NewIdx);
							});
					}
						
					
				}
			}
		}

		const int32 NumTransforms = EvaluatedResult.Transform.Num();
		for(int32 Index = 0; Index < NumTransforms; ++Index)
		{
			const int32      ParticleIndex      = EvaluatedResult.ParticleIndices[Index];
			const FTransform EvaluatedTransform = EvaluatedResult.Transform[Index];

			if (Chaos::FPBDRigidClusteredParticleHandle* Handle = Proxy->GetParticle_Internal(ParticleIndex))
			{
				if(Handle->ObjectState() != EObjectStateType::Kinematic)
				{
					// If a field or other external actor set the particle to static or dynamic we no longer apply the cache
					continue;
				}

				// EvaluatedTransform is in world space already as we have passed MassToLocal transform to InCache->Evaluate()
				// to make sure the interpolation was done correctly in world space  
				const FTransform WorldTransform{EvaluatedTransform};

				if (GeometryCollectionCacheAdapter::CVars::bUseKinematicTarget)
				{
					FKinematicTarget Target = FKinematicTarget::MakePositionTarget(WorldTransform);
					Solver->GetEvolution()->SetParticleKinematicTarget(Handle, Target);
				}
				else
				{
					Handle->SetP(WorldTransform.GetTranslation());
					Handle->SetQ(WorldTransform.GetRotation());
					Handle->SetX(Handle->GetP());
					Handle->SetRf(Handle->GetQf());

					Handle->UpdateWorldSpaceState(WorldTransform, FVec3(0));

					if (!Handle->Disabled())
					{
						Solver->GetEvolution()->DirtyParticle(*Handle);
					}
				}

				if(FRigidParticle* ClusterParent = Handle->ClusterIds().Id)
				{
					if(FClusterParticle* Parent = ClusterParent->CastToClustered())
					{
						if(Parent->InternalCluster())
						{
							// This is an unmanaged particle. Because its children are kinematic it will be also.
							// however we need to update its position at least once to place it correctly.
							// The child was placed with:
							//     ChildT = ChildHandle->ChildToParent() * FTransform(ParentHandle->R(), ParentHandle->X());
							// When it was simulated, so we can work backwards to place the parent.
							// This will result in multiple transform sets happening to the parent but allows us to mostly ignore
							// that it exists, if it doesn't the child still gets set to the correct position.
							FTransform ChildTransform = Handle->ChildToParent();
							FTransform Result = ChildTransform.Inverse() * WorldTransform;

							if (GeometryCollectionCacheAdapter::CVars::bUseKinematicTarget)
							{
								FKinematicTarget Target = FKinematicTarget::MakePositionTarget(Result);
								Solver->GetEvolution()->SetParticleKinematicTarget(Parent, Target);
							}
							else
							{
								Parent->SetP(Result.GetTranslation());
								Parent->SetX(Result.GetTranslation());
								Parent->SetQ(Result.GetRotation());
								Parent->SetR(Result.GetRotation());

								Parent->UpdateWorldSpaceState(Result, FVec3(0));

								if (!Parent->Disabled())
								{
									Solver->GetEvolution()->DirtyParticle(*Parent);
								}
							}
						}
					}
				}

				OutUpdatedRigids.Add(Handle);
			}
		}
	}

	bool FGeometryCollectionCacheAdapter::ValidForPlayback(UPrimitiveComponent* InComponent, UChaosCache* InCache) const
	{
		UGeometryCollectionComponent*  GeomComponent = Cast<UGeometryCollectionComponent>(InComponent);

		if(!GeomComponent)
		{
			return false;
		}

		const UGeometryCollection* Collection = GeomComponent->RestCollection;

		if(!Collection || !Collection->GetGeometryCollection().IsValid())
		{
			return false;
		}

		// Really permissive check - as long as we can map all tracks to a particle in the geometry collection we'll allow this to play.
		// allows geometry changes without invalidating an entire cache on reimport or modification.
		const int32 NumTransforms = Collection->GetGeometryCollection()->Transform.Num();
		for(const int32 ParticleIndex : InCache->TrackToParticle)
		{
			if(ParticleIndex < 0 || ParticleIndex >= NumTransforms)
			{
				return false;
			}
		}

		return true;
	}

	FGuid FGeometryCollectionCacheAdapter::GetGuid() const
	{
		FGuid NewGuid;
		checkSlow(FGuid::Parse(TEXT("A3147746B50C47C883B93DBF85CBB589"), NewGuid));
		return NewGuid;
	}

	Chaos::FPhysicsSolver* FGeometryCollectionCacheAdapter::GetComponentSolver(UPrimitiveComponent* InComponent) const
	{
		// If the observed component is a Geometry Collection using a non-default Chaos solver..
		if (UGeometryCollectionComponent* GeometryCollectionComponent = Cast<UGeometryCollectionComponent>(InComponent))
		{
			if (AChaosSolverActor* SolverActor = GeometryCollectionComponent->GetPhysicsSolverActor())
			{
				return SolverActor->GetSolver();
			}			
		}

		// ..otherwise use the default solver.
		if (InComponent && InComponent->GetWorld())
		{
			UWorld* ComponentWorld = InComponent->GetWorld();

			if (FPhysScene* WorldScene = ComponentWorld->GetPhysicsScene())
			{
				return WorldScene->GetSolver();
			}
		}


		return nullptr;
	}
	
	Chaos::FPhysicsSolverEvents* FGeometryCollectionCacheAdapter::BuildEventsSolver(UPrimitiveComponent* InComponent) const
	{
		return GetComponentSolver(InComponent);
	}

	void FGeometryCollectionCacheAdapter::SetRestState(UPrimitiveComponent* InComponent, UChaosCache* InCache, const FTransform& InRootTransform, Chaos::FReal InTime) const
	{
		// Caches recorded previous to Version 1 may not scrub correctly as the MassToLocal transform has been burned in.
		
		EnsureIsInGameThreadContext();

		
		if (!InCache || InCache->GetDuration() == 0.0f)
		{
			return;
		}

		if (UGeometryCollectionComponent* Comp = CastChecked<UGeometryCollectionComponent>(InComponent))
		{
			const FTransform ActorToWorld = InRootTransform;
			FTransform WorldToComponent = Comp->GetComponentTransform().Inverse();
			FTransform ActorToComponent = ActorToWorld * WorldToComponent;
			const FTransform CompScaleTransform = FTransform(FQuat::Identity, FVector::ZeroVector, Comp->GetComponentTransform().GetScale3D());
			const bool bIsScaledComponent = !CompScaleTransform.GetScale3D().Equals(FVector::OneVector);

			if (const UGeometryCollection* RestCollection = Comp->GetRestCollection())
			{
				TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollection = RestCollection->GetGeometryCollection();
				const int32 NumTransforms = GeometryCollection->NumElements(FGeometryCollection::TransformGroup);

				FPlaybackTickRecord TickRecord;
				TickRecord.SetLastTime(InTime);
				FCacheEvaluationContext Context(TickRecord);
				Context.bEvaluateTransform = true;
				Context.bEvaluateCurves = false;
				Context.bEvaluateEvents = false;

				// we only need the rest transforms, so no need to pass MassToLocal transforms 
				FCacheEvaluationResult EvaluatedResult = InCache->Evaluate(Context, nullptr);
				const int32 NumCacheTransforms = EvaluatedResult.Transform.Num();
			
				// Any bone that is not explicitly set by the cache defaults to it's rest collection position.
				TArray<FTransform> RestTransforms;
				RestTransforms.SetNum(NumTransforms);
				for (int32 Idx = 0; Idx < NumTransforms; ++Idx)
				{
					RestTransforms[Idx] = FTransform(GeometryCollection->Transform[Idx]);
				}

				for (int32 CacheIdx = 0; CacheIdx < NumCacheTransforms; ++CacheIdx)
				{
					const int32 TransformIdx = EvaluatedResult.ParticleIndices[CacheIdx];
					RestTransforms[TransformIdx] = EvaluatedResult.Transform[CacheIdx] * ActorToComponent;
				}

				if (bIsScaledComponent)
				{
					const TManagedArray<FTransform>& MassToLocal = GeometryCollection->GetAttribute<FTransform>("MassToLocal", FGeometryCollection::TransformGroup);
					for (int32 CacheIdx = 0; CacheIdx < NumCacheTransforms; ++CacheIdx)
					{
						const int32 TransformIdx = EvaluatedResult.ParticleIndices[CacheIdx];
						FTransform MassToLocalTransform = MassToLocal[TransformIdx];
						RestTransforms[TransformIdx] = MassToLocalTransform.Inverse() * CompScaleTransform * MassToLocalTransform * RestTransforms[TransformIdx];
					}
				}

				// Set any broken clusters to identity so that they no longer influence child transforms.
				TArray<int32> ReleaseIndices = GatherAllBreaksUpToTime(InCache, InTime);
				const TManagedArray<int32>& Parent = GeometryCollection->Parent;
				const int32 RootIndex = Comp->GetRootIndex();
				bool bIsRootBroken = false;
				for (int32 ReleaseIdx : ReleaseIndices)
				{
					const int32 ClusterIdx = Parent[ReleaseIdx];
					if (ClusterIdx == RootIndex)
					{
						bIsRootBroken = true;
					}
					if (ClusterIdx > INDEX_NONE)
					{
						RestTransforms[ClusterIdx] = FTransform::Identity;
					}
				}

				if (bIsRootBroken != Comp->IsForceBrokenForCustomRenderer())
				{
					// this is the root bone broken we need to disable the root proxy
					Comp->ForceBrokenForCustomRenderer(bIsRootBroken);
				}

				Comp->SetRestState(MoveTemp(RestTransforms));
			}
		}
	}

	void FGeometryCollectionCacheAdapter::EmitEvents(UPrimitiveComponent* InComponent, UChaosCache* InCache, FPlaybackTickRecord& TickRecord) const
	{
		EnsureIsInGameThreadContext();

		if (!InCache || InCache->GetDuration() == 0.0f)
		{
			return;
		}

		if (UGeometryCollectionComponent* Comp = CastChecked<UGeometryCollectionComponent>(InComponent))
		{
			FCacheEvaluationContext Context(TickRecord);
			Context.bEvaluateTransform = false;
			Context.bEvaluateCurves = false;
			Context.bEvaluateEvents = true;

			FCacheEvaluationResult EvaluatedResult = InCache->Evaluate(Context, nullptr);

			EmitBreakingEvents_GameThread(*Comp, EvaluatedResult);
			EmitCollisionEvents_GameThread(*Comp, EvaluatedResult);
		}
	}

	void FGeometryCollectionCacheAdapter::EmitCollisionEvents_GameThread(UPrimitiveComponent& InComponent, const FCacheEvaluationResult& Results) const
	{
		if (const TArray<FCacheEventHandle>* CollisionEvents = Results.Events.Find(FCollisionEvent::EventName))
		{
			UWorld* ComponentWorld = InComponent.GetWorld();
			if (FPhysScene* PhysicsScene = ComponentWorld ? ComponentWorld->GetPhysicsScene() : nullptr)
			{
				UChaosEventRelay* ChaosEventRelay = PhysicsScene->GetChaosEventRelay();
				if (ChaosEventRelay && ChaosEventRelay->OnCollisionEvent.IsBound())
				{
					const FTransform ComponentToWorld = InComponent.GetComponentToWorld();

					TArray<FCollisionChaosEvent> CollisionsEventsToEmit;
					for (const FCacheEventHandle& Handle : *CollisionEvents)
					{
						if (const FCollisionEvent* CachedEvent = Handle.GetConst<FCollisionEvent>())
						{
							FCollisionChaosEvent& CollisionEvent = CollisionsEventsToEmit.AddDefaulted_GetRef();
							CollisionEvent.Location = ComponentToWorld.TransformPosition(CachedEvent->Location);
							CollisionEvent.AccumulatedImpulse = ComponentToWorld.TransformVectorNoScale(CachedEvent->AccumulatedImpulse);
							CollisionEvent.Normal = ComponentToWorld.TransformVectorNoScale(CachedEvent->Normal);
							CollisionEvent.PenetrationDepth = CachedEvent->PenetrationDepth;

							// since we don't know what component each body comes from, we set both to the same
							// this will allow at least the event handler to be able to query component related information like materials
							CollisionEvent.Body1.Component = &InComponent; 
							CollisionEvent.Body2.Component = &InComponent;

							CollisionEvent.Body1.Velocity = ComponentToWorld.TransformVectorNoScale(CachedEvent->Velocity1);
							CollisionEvent.Body1.DeltaVelocity = ComponentToWorld.TransformVectorNoScale(CachedEvent->DeltaVelocity1);
							CollisionEvent.Body1.AngularVelocity = ComponentToWorld.TransformVectorNoScale(CachedEvent->AngularVelocity1);
							CollisionEvent.Body1.Mass = CachedEvent->Mass1;
							CollisionEvent.Body1.PhysMaterial = nullptr; // non-cached information
							CollisionEvent.Body1.BodyIndex = INDEX_NONE; // non-cached information
							CollisionEvent.Body1.BoneName = NAME_None; // non-cached information

							CollisionEvent.Body2.Velocity = ComponentToWorld.TransformVectorNoScale(CachedEvent->Velocity2);
							CollisionEvent.Body2.DeltaVelocity = ComponentToWorld.TransformVectorNoScale(CachedEvent->DeltaVelocity2);
							CollisionEvent.Body2.AngularVelocity = ComponentToWorld.TransformVectorNoScale(CachedEvent->AngularVelocity2);
							CollisionEvent.Body2.Mass = CachedEvent->Mass2;
							CollisionEvent.Body2.PhysMaterial = nullptr; // non-cached information
							CollisionEvent.Body2.BodyIndex = INDEX_NONE; // non-cached information
							CollisionEvent.Body2.BoneName = NAME_None; // non-cached information
						}
					}
					if (CollisionsEventsToEmit.Num() > 0)
					{
						ChaosEventRelay->DispatchPhysicsCollisionEvents(CollisionsEventsToEmit);
					}
				}
			}
		}
	}

	void FGeometryCollectionCacheAdapter::EmitBreakingEvents_GameThread(UPrimitiveComponent& InComponent, const FCacheEvaluationResult& Results) const
	{
		if (const TArray<FCacheEventHandle>* BreakingEvents = Results.Events.Find(FBreakingEvent::EventName))
		{
			UWorld* ComponentWorld = InComponent.GetWorld();
			if (FPhysScene* PhysicsScene = ComponentWorld ? ComponentWorld->GetPhysicsScene() : nullptr)
			{
				UChaosEventRelay* ChaosEventRelay = PhysicsScene->GetChaosEventRelay();
				if (ChaosEventRelay && ChaosEventRelay->OnBreakEvent.IsBound())
				{
					const FTransform ComponentToWorld = InComponent.GetComponentToWorld();

					TArray<FChaosBreakEvent> BreakEventsToEmit;
					for (const FCacheEventHandle& Handle : *BreakingEvents)
					{
						if (const FBreakingEvent* CachedEvent = Handle.GetConst<FBreakingEvent>())
						{
							FChaosBreakEvent& BreakEvent = BreakEventsToEmit.AddDefaulted_GetRef();
							BreakEvent.Component = &InComponent;
							BreakEvent.Location = ComponentToWorld.TransformPosition(CachedEvent->Location);
							BreakEvent.Orientation = ComponentToWorld.TransformRotation(CachedEvent->Orientation);
							BreakEvent.Velocity = ComponentToWorld.TransformVectorNoScale(CachedEvent->Velocity);
							BreakEvent.AngularVelocity = ComponentToWorld.TransformVectorNoScale(CachedEvent->AngularVelocity);
							const FVector Extents = (CachedEvent->BoundingBoxMax - CachedEvent->BoundingBoxMin) * 0.5;
							const FBox Bounds(CachedEvent->Location - Extents, CachedEvent->Location + Extents);
							const FBox TransformedBounds = Bounds.TransformBy(ComponentToWorld);
							BreakEvent.Extents = TransformedBounds.GetExtent();
							BreakEvent.Mass = CachedEvent->Mass;
							BreakEvent.Index = CachedEvent->Index;
							BreakEvent.bFromCrumble = false; // todo(chaos) : maybe we should cache this informnation too in the future ? 
						}
					}
					if (BreakEventsToEmit.Num() > 0)
					{
						ChaosEventRelay->DispatchPhysicsBreakEvents(BreakEventsToEmit);
					}
				}
			}
		}
	}

	void FGeometryCollectionCacheAdapter::RegisterEventHandler(FGeometryCollectionPhysicsProxy* Proxy)
	{
		if (!Proxy)
		{
			return;
		}

		if (Chaos::FPhysicsSolver* Solver = Proxy->GetSolver<Chaos::FPhysicsSolver>())
		{
			const FSimulationParameters& SimulationParameters = Proxy->GetSimParameters();

			// this method ensure that only the necessary handlers are registered by solver and for the need of the specific proxies we are tracking
			if (Chaos::FEventManager* EventManager = Solver->GetEventManager())
			{
				if (SimulationParameters.bGenerateCollisionData || SimulationParameters.bGenerateGlobalCollisionData)
				{
					const FSolverEventPair SolverCollisionEventPair{ Solver,Chaos::EEventType::Collision };
					if (!SolversListenedToForEvents.Contains(SolverCollisionEventPair))
					{
						EventManager->RegisterHandler<Chaos::FCollisionEventData>(Chaos::EEventType::Collision, this, &FGeometryCollectionCacheAdapter::HandleCollisionEvents);
						SolversListenedToForEvents.Add(SolverCollisionEventPair);
					}
				}
				if (SimulationParameters.bGenerateBreakingData || SimulationParameters.bGenerateGlobalBreakingData)
				{
					const FSolverEventPair SolverBreakingEventPair{ Solver,Chaos::EEventType::Breaking };
					if (!SolversListenedToForEvents.Contains(SolverBreakingEventPair))
					{
						EventManager->RegisterHandler<Chaos::FBreakingEventData>(Chaos::EEventType::Breaking, this, &FGeometryCollectionCacheAdapter::HandleBreakingEvents);
						SolversListenedToForEvents.Add(SolverBreakingEventPair);
					}
				}
				if (SimulationParameters.bGenerateTrailingData || SimulationParameters.bGenerateGlobalTrailingData)
				{
					const FSolverEventPair SolverTrailingEventPair{ Solver,Chaos::EEventType::Trailing};
					if (!SolversListenedToForEvents.Contains(SolverTrailingEventPair))
					{
						EventManager->RegisterHandler<Chaos::FTrailingEventData>(Chaos::EEventType::Trailing, this, &FGeometryCollectionCacheAdapter::HandleTrailingEvents);
						SolversListenedToForEvents.Add(SolverTrailingEventPair);
					}
				}
			}
		}
	}

	void FGeometryCollectionCacheAdapter::UnregisterEventHandler(FGeometryCollectionPhysicsProxy* Proxy)
	{
		if (!Proxy)
		{
			return;
		}

		if (Chaos::FPhysicsSolver* Solver = Proxy->GetSolver<Chaos::FPhysicsSolver>())
		{
			// find all proxies for this Solver and check what they need to listen to after removal of the current proxy
			bool bListenToCollisionEvents = false;
			bool bListenToBreakingEvents = false;
			bool bListenToTrailingEvents = false;
			for (const TPair<IPhysicsProxyBase*, FCachedEventData>& CacheDataPair : CachedData)
			{
				const IPhysicsProxyBase* OtherProxy = CacheDataPair.Key;
				const FGeometryCollectionPhysicsProxy* OtherGCProxy =
					OtherProxy->GetType() == FGeometryCollectionPhysicsProxy::ConcreteType()
					? static_cast<const FGeometryCollectionPhysicsProxy*>(OtherProxy)
					: nullptr;

				// exclude the proxy we are about to remove 
				if (Proxy != OtherGCProxy && OtherGCProxy)
				{
					const FSimulationParameters& OtherSimParams = OtherGCProxy->GetSimParameters();
					if (OtherProxy->GetSolver<Chaos::FPhysicsSolver>() == Solver)
					{
						bListenToCollisionEvents |= (OtherSimParams.bGenerateCollisionData || OtherSimParams.bGenerateGlobalCollisionData);
						bListenToBreakingEvents |= (OtherSimParams.bGenerateBreakingData || OtherSimParams.bGenerateGlobalBreakingData);
						bListenToTrailingEvents |= (OtherSimParams.bGenerateTrailingData || OtherSimParams.bGenerateGlobalTrailingData);
					}
				}
			}

			// unregister the handlers that we do not need anymore
			if (Chaos::FEventManager* EventManager = Solver->GetEventManager())
			{
				if (!bListenToCollisionEvents)
				{
					const FSolverEventPair SolverCollisionEventPair{ Solver,Chaos::EEventType::Collision };
					EventManager->UnregisterHandler(Chaos::EEventType::Collision, this);
					SolversListenedToForEvents.Remove(SolverCollisionEventPair);
				}
				if (!bListenToBreakingEvents)
				{
					const FSolverEventPair SolverBreakingEventPair{ Solver,Chaos::EEventType::Breaking };
					EventManager->UnregisterHandler(Chaos::EEventType::Breaking, this);
					SolversListenedToForEvents.Remove(SolverBreakingEventPair);
				}
				if (!bListenToTrailingEvents)
				{
					const FSolverEventPair SolverTrailingEventPair{ Solver,Chaos::EEventType::Trailing };
					EventManager->UnregisterHandler(Chaos::EEventType::Trailing, this);
					SolversListenedToForEvents.Remove(SolverTrailingEventPair);
				}
			}
		}
	}

	bool FGeometryCollectionCacheAdapter::InitializeForRecord(UPrimitiveComponent* InComponent, UChaosCache* InCache)
	{
		EnsureIsInGameThreadContext();
		
		UGeometryCollectionComponent*    Comp     = CastChecked<UGeometryCollectionComponent>(InComponent);
		FGeometryCollectionPhysicsProxy* Proxy = Comp ? Comp->GetPhysicsProxy() : nullptr;

		if (!Proxy)
		{
			return false;
		}
		if (Comp)
		{
			// make sure we have the root proxy enabled if any
			Comp->ForceBrokenForCustomRenderer(false);
		}

		Chaos::FPhysicsSolver* Solver = Proxy->GetSolver<Chaos::FPhysicsSolver>();
		if (!Solver)
		{
			return false;
		}

		// In case commands have been issued that conflict with our requirement to generate data, flush the queue
		Solver->AdvanceAndDispatch_External(0);
		Solver->WaitOnPendingTasks_External();

		// We need secondary event data to record event information into the cache
		Solver->EnqueueCommandImmediate([Solver]()
			{
				Solver->SetGenerateBreakingData(true);
				Solver->SetGenerateCollisionData(true);
				Solver->SetGenerateTrailingData(true);
			});

		
		RegisterEventHandler(Proxy);

		// we only need to clear the data for the first proxy we register
		if (CachedData.Num() == 0)
		{
			BreakingDataArray = nullptr;
			CollisionDataArray = nullptr;
			TrailingDataArray = nullptr;
		}

		FCachedEventData& CachedEventData = CachedData.FindOrAdd(Proxy);
		CachedEventData.ProxyBreakingDataIndices = nullptr;
		CachedEventData.ProxyCollisionDataIndices = nullptr;
		CachedEventData.ProxyTrailingDataIndices = nullptr;

		return true;
	}

	bool FGeometryCollectionCacheAdapter::InitializeForPlayback(UPrimitiveComponent* InComponent, UChaosCache* InCache, float InTime)
	{
		EnsureIsInGameThreadContext();

		if (UGeometryCollectionComponent* Comp = CastChecked<UGeometryCollectionComponent>(InComponent))
		{
			if (const UGeometryCollection* RestCollection = Comp->GetRestCollection())
			{
				// Set up initial conditions... 
				
				TArray<FTransform> InitialTransforms;
				{
					FPlaybackTickRecord TickRecord;
					TickRecord.SetLastTime(InTime);
					FCacheEvaluationContext Context(TickRecord);
					Context.bEvaluateTransform = true;
					Context.bEvaluateCurves = false;
					Context.bEvaluateEvents = false;

					// initial transforms do no need to be multiplied by MassToLocal transforms so we can pass nullptr
					FCacheEvaluationResult EvaluatedResult = InCache->Evaluate(Context, nullptr);
					const int32 NumCacheTransforms = EvaluatedResult.Transform.Num();

					// Any bone that is not explicitly set by the cache defaults to it's rest collection position.
					TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollection = RestCollection->GetGeometryCollection();
					const int32 NumTransforms = GeometryCollection->NumElements(FGeometryCollection::TransformGroup);

					InitialTransforms.SetNum(NumTransforms);
					for (int32 Idx = 0; Idx < NumTransforms; ++Idx)
					{
						InitialTransforms[Idx] = FTransform(GeometryCollection->Transform[Idx]);
					}

					for (int32 CacheIdx = 0; CacheIdx < NumCacheTransforms; ++CacheIdx)
					{
						const int32 TransformIdx = EvaluatedResult.ParticleIndices[CacheIdx];
						InitialTransforms[TransformIdx] = EvaluatedResult.Transform[CacheIdx];
					}
				}

				// make sure we have the root proxy enabled if any
				Comp->ForceBrokenForCustomRenderer(false);

				Comp->SetInitialTransforms(InitialTransforms);
					
				TArray<int32> ReleaseIndices = GatherAllBreaksUpToTime(InCache, InTime);
				Comp->SetInitialClusterBreaks(ReleaseIndices);

				Comp->SetDynamicState(Chaos::EObjectStateType::Kinematic);
		
				// ...then initialize the proxy and particles.
				Comp->SetSimulatePhysics(true);

				return true;
			}
		}

		return false;
	}

	void FGeometryCollectionCacheAdapter::FinalizeComponent(UPrimitiveComponent* InComponent, UChaosCache* InCache)
	{
		EnsureIsInGameThreadContext();

		// make sure we remove the proxy from the list to avoid accumulating them record after record
		if (UGeometryCollectionComponent* Comp = CastChecked<UGeometryCollectionComponent>(InComponent))
		{
			if (FGeometryCollectionPhysicsProxy* Proxy = Comp->GetPhysicsProxy())
			{
				UnregisterEventHandler(Proxy);
				CachedData.Remove(Proxy);
			}
		}
	}

	TArray<int32> FGeometryCollectionCacheAdapter::GatherAllBreaksUpToTime(UChaosCache* InCache, float InTime) const
	{
		// Evaluate all breaking event that have occurred from the beginning of the cache up to the specified time.
		TArray<int32> ReleaseIndices;

		FPlaybackTickRecord TickRecord;
		TickRecord.SetLastTime(0.0);
		TickRecord.SetDt(InTime);
		FCacheEvaluationContext Context(TickRecord);
		Context.bEvaluateTransform = false;
		Context.bEvaluateCurves = false;
		Context.bEvaluateEvents = true;

		// passing nullptr for the MassToLocal as we do not need to evaluate the transforms
		FCacheEvaluationResult EvaluatedResult = InCache->Evaluate(Context, nullptr);
		const TArray<FCacheEventHandle>* EnableEvents = EvaluatedResult.Events.Find(FEnableStateEvent::EventName);
		if (EnableEvents)
		{
			ReleaseIndices.Reserve(EnableEvents->Num());
			for (const FCacheEventHandle& Handle : *EnableEvents)
			{
				if (const FEnableStateEvent* Event = Handle.GetConst<FEnableStateEvent>())
				{
					if (Event->bEnable)
					{
						ReleaseIndices.Add(Event->Index);
					}
				}
			}
		}

		return ReleaseIndices;
	}

	void FGeometryCollectionCacheAdapter::HandleBreakingEvents(const Chaos::FBreakingEventData& Event)
	{
		EnsureIsInGameThreadContext();
		
		BreakingDataArray = &Event.BreakingData.AllBreakingsArray;

		for (TPair<IPhysicsProxyBase*, FCachedEventData>& Data : CachedData)
		{
			if (Event.PhysicsProxyToBreakingIndices.PhysicsProxyToIndicesMap.Contains(Data.Key))
			{
				Data.Value.ProxyBreakingDataIndices = &Event.PhysicsProxyToBreakingIndices.PhysicsProxyToIndicesMap[Data.Key];
			}
			else
			{
				Data.Value.ProxyBreakingDataIndices = nullptr;
			}
		}
	}

	void FGeometryCollectionCacheAdapter::HandleCollisionEvents(const Chaos::FCollisionEventData& Event)
	{
		EnsureIsInGameThreadContext();
		
		CollisionDataArray = &Event.CollisionData.AllCollisionsArray;

		for (TPair<IPhysicsProxyBase*, FCachedEventData>& Data : CachedData)
		{
			if (Event.PhysicsProxyToCollisionIndices.PhysicsProxyToIndicesMap.Contains(Data.Key))
			{
				Data.Value.ProxyCollisionDataIndices = &Event.PhysicsProxyToCollisionIndices.PhysicsProxyToIndicesMap[Data.Key];
			}
			else
			{
				Data.Value.ProxyCollisionDataIndices = nullptr;
			}
		}
	}

	void FGeometryCollectionCacheAdapter::HandleTrailingEvents(const Chaos::FTrailingEventData& Event)
	{
		EnsureIsInGameThreadContext();
		
		TrailingDataArray = &Event.TrailingData.AllTrailingsArray;

		for (TPair<IPhysicsProxyBase*, FCachedEventData>& Data : CachedData)
		{
			if (Event.PhysicsProxyToTrailingIndices.PhysicsProxyToIndicesMap.Contains(Data.Key))
			{
				Data.Value.ProxyTrailingDataIndices = &Event.PhysicsProxyToTrailingIndices.PhysicsProxyToIndicesMap[Data.Key];
			}
			else
			{
				Data.Value.ProxyTrailingDataIndices = nullptr;
			}
		}
	}

}    // namespace Chaos

