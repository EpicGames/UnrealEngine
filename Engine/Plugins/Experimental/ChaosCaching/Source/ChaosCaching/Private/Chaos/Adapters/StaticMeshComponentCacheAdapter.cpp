// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Adapters/StaticMeshComponentCacheAdapter.h"
#include "Chaos/Adapters/CacheAdapter.h"
#include "Engine/World.h"
#include "Physics/Experimental/PhysInterface_Chaos.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "PBDRigidsSolver.h"
#include "Chaos/ChaosCache.h"
#include "Components/StaticMeshComponent.h"
#include "Physics/Experimental/PhysScene_Chaos.h"


namespace Chaos
{

	FComponentCacheAdapter::SupportType FStaticMeshCacheAdapter::SupportsComponentClass(UClass* InComponentClass) const
	{
		UClass* Desired = GetDesiredClass();
		if(InComponentClass == Desired)
		{
			return FComponentCacheAdapter::SupportType::Direct;
		}
		else if(InComponentClass->IsChildOf(Desired))
		{
			return FComponentCacheAdapter::SupportType::Derived;
		}

		return FComponentCacheAdapter::SupportType::None;
	}

	UClass* FStaticMeshCacheAdapter::GetDesiredClass() const
	{
		return UStaticMeshComponent::StaticClass();
	}

	uint8 FStaticMeshCacheAdapter::GetPriority() const
	{
		return EngineAdapterPriorityBegin;
	}

	void RecordToCacheInternal(FSingleParticlePhysicsProxy* InProxy, const FTransform& InRootTransform, FPendingFrameWrite& OutFrame, Chaos::FReal InTime)
	{
		if (!InProxy)
		{
			return;
		}

		if(TGeometryParticleHandle<FReal,3>* Handle = InProxy->GetHandle_LowLevel())
		{
			if(FPBDRigidParticleHandle* AsRigid = Handle->CastToRigidParticle())
			{
				FPendingParticleWrite NewData;

				NewData.ParticleIndex = 0; // Only one particle for static caches
				NewData.PendingTransform = FTransform(AsRigid->GetR(), AsRigid->GetX()).GetRelativeTransform(InRootTransform);

				OutFrame.PendingParticleData.Add(MoveTemp(NewData));
			}
		}
	}

	void FStaticMeshCacheAdapter::Record_PostSolve(UPrimitiveComponent* InComponent, const FTransform& InRootTransform, FPendingFrameWrite& OutFrame, Chaos::FReal InTime) const
	{
		UStaticMeshComponent* MeshComp = CastChecked<UStaticMeshComponent>(InComponent);

		FSingleParticlePhysicsProxy* PhysProxy = MeshComp->BodyInstance.GetPhysicsActor();

		RecordToCacheInternal(PhysProxy, InRootTransform, OutFrame, InTime);
	}

	void PlayFromCacheInternal(FSingleParticlePhysicsProxy* InProxy, UChaosCache* InCache, FPlaybackTickRecord& TickRecord, TArray<TPBDRigidParticleHandle<Chaos::FReal, 3>*>& OutUpdatedRigids) 
	{
		if(!InCache || InCache->GetDuration() == 0.0f || !InProxy)
		{
			return;
		}

		TGeometryParticleHandle<FReal,3>* Handle = InProxy->GetHandle_LowLevel();
		
		if(Handle && Handle->ObjectState() == EObjectStateType::Kinematic)
		{
			if(FPBDRigidParticleHandle* AsRigid = Handle->CastToRigidParticle())
			{
				FCacheEvaluationContext Context(TickRecord);
				Context.bEvaluateTransform = true;
				Context.bEvaluateCurves = false;
				Context.bEvaluateEvents = false;

				// Note: static mesh don't use the concept of MassToLocal transforms, passing nullptr
				FCacheEvaluationResult EvaluatedResult = InCache->Evaluate(Context, nullptr);

				// Either 0 or 1 result, 0 for nothing in the eval track - 1 if there was.
				if(EvaluatedResult.Transform.Num() == 1)
				{
					AsRigid->SetX(EvaluatedResult.Transform[0].GetTranslation());
					AsRigid->SetR(EvaluatedResult.Transform[0].GetRotation());
				}

				OutUpdatedRigids.Add(AsRigid);
			}
		}
	}

	void FStaticMeshCacheAdapter::Playback_PreSolve(UPrimitiveComponent* InComponent, UChaosCache* InCache, Chaos::FReal InTime, FPlaybackTickRecord& TickRecord, TArray<TPBDRigidParticleHandle<Chaos::FReal, 3>*>& OutUpdatedRigids) const
	{
		UStaticMeshComponent* MeshComp = CastChecked<UStaticMeshComponent>(InComponent);

		FSingleParticlePhysicsProxy* PhysProxy = MeshComp->BodyInstance.GetPhysicsActor();

		PlayFromCacheInternal(PhysProxy, InCache, TickRecord, OutUpdatedRigids);
	}

	FGuid FStaticMeshCacheAdapter::GetGuid() const
	{
		FGuid NewGuid;
		checkSlow(FGuid::Parse(TEXT("82570E6C014B4D2FA7866A0EC99924C4"), NewGuid));
		return NewGuid;
	}

	bool FStaticMeshCacheAdapter::ValidForPlayback(UPrimitiveComponent* InComponent, UChaosCache* InCache) const
	{
		// If we have a mesh we can play back any cache as long as it has one or more tracks
		UStaticMeshComponent* Comp = Cast<UStaticMeshComponent>(InComponent);
		return Comp && Comp->GetStaticMesh() && InCache->TrackToParticle.Num() > 0;
	}

	Chaos::FPhysicsSolver* FStaticMeshCacheAdapter::GetComponentSolver(UPrimitiveComponent* InComponent) const
	{
		if(InComponent && InComponent->GetWorld())
		{
			UWorld* ComponentWorld = InComponent->GetWorld();
			
			if(FPhysScene* WorldScene = ComponentWorld->GetPhysicsScene())
			{
				return WorldScene->GetSolver();
			}
		}

		return nullptr;
	}
	
	Chaos::FPhysicsSolverEvents* FStaticMeshCacheAdapter::BuildEventsSolver(UPrimitiveComponent* InComponent) const
	{
		return GetComponentSolver(InComponent);
	}
	
	void FStaticMeshCacheAdapter::SetRestState(UPrimitiveComponent* InComponent, UChaosCache* InCache, const FTransform& InRootTransform, Chaos::FReal InTime) const
	{
		if (!InCache || InCache->GetDuration() == 0.0f)
		{
			return;
		}

		if (InCache->ParticleTracks.Num() == 1)
		{

			FPlaybackTickRecord TickRecord;
			TickRecord.SetLastTime(InTime);
			FCacheEvaluationContext Context(TickRecord);
			Context.bEvaluateTransform = true;
			Context.bEvaluateCurves = false;
			Context.bEvaluateEvents = false;

			FTransform RestTransform;
			InCache->EvaluateSingle(0, TickRecord, nullptr, &RestTransform, nullptr);

			// Evaluated transform is in CacheManager space.
			InComponent->SetWorldTransform(RestTransform * InRootTransform, false);
		}
	}
	
	bool FStaticMeshCacheAdapter::InitializeForRecord(UPrimitiveComponent* InComponent, UChaosCache* InCache)
	{
		return true;
	}

	bool FStaticMeshCacheAdapter::InitializeForPlayback(UPrimitiveComponent* InComponent, UChaosCache* InCache, float InTime)
	{
		EnsureIsInGameThreadContext();

		if(Cast<UStaticMeshComponent>(InComponent) && InComponent->GetBodyInstance() && InComponent->GetBodyInstance()->GetPhysicsActor())
		{
			FPhysInterface_Chaos::SetIsKinematic_AssumesLocked(InComponent->GetBodyInstance()->GetPhysicsActor(), true);
		}

		return true;
	}

}    // namespace Chaos
