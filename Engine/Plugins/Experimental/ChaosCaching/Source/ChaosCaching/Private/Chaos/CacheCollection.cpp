// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/CacheCollection.h"
#include "Chaos/ChaosCache.h"
#include "Algo/Find.h"
#include "Async/ParallelFor.h"
#include "UObject/AssetRegistryTagsContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CacheCollection)

void UChaosCacheCollection::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UChaosCacheCollection::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
#if WITH_EDITOR
	int32 NumCaches = Caches.Num();

	Context.AddTag(FAssetRegistryTag(TEXT("Number of Observed Components"), FString::FromInt(NumCaches), FAssetRegistryTag::TT_Numerical));

	float MaxDuration = 0.0;
	uint32 MaxRecordedFrames = 0;
	int64 TotalTracks = 0;
	int64 TotalTransformKeys = 0;
	for (int32 CacheIdx = 0; CacheIdx < NumCaches; ++CacheIdx)
	{
		if (UChaosCache* Cache = Caches[CacheIdx])
		{
			MaxDuration = FMath::Max(MaxDuration, Cache->RecordedDuration);
			MaxRecordedFrames = FMath::Max(MaxRecordedFrames, Cache->NumRecordedFrames);
			TotalTracks += Cache->ParticleTracks.Num();
			for (const FPerParticleCacheData& Track : Cache->ParticleTracks)
			{
				TotalTransformKeys += Track.TransformData.GetNumKeys();
			}
		}
	}	
	const int64 TransformStorageSizeInBytes = sizeof(FVector3f) + sizeof(FQuat4f) + sizeof(FVector3f) + sizeof(float); // pos + rot + scale + timestamp
	const double EstimatedTransformMemoryInMegaBytes = double(TotalTransformKeys * TransformStorageSizeInBytes) / 1024.0 / 1024.0;

	Context.AddTag(FAssetRegistryTag(TEXT("Recorded Duration"), FString::Printf(TEXT("%.2f"), MaxDuration), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag(TEXT("Recorded Frames"), FString::Printf(TEXT("%d"), MaxRecordedFrames), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag(TEXT("Total Tracks"), FString::Printf(TEXT("%lld"), TotalTracks), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag(TEXT("Total Transform Keys"), FString::Printf(TEXT("%lld"), TotalTransformKeys), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag(TEXT("Estimated Transform Memory"), FString::Printf(TEXT("%.2f MB"), EstimatedTransformMemoryInMegaBytes), FAssetRegistryTag::TT_Numerical));
#endif

	Super::GetAssetRegistryTags(Context);
}

UChaosCache* UChaosCacheCollection::FindCache(const FName& CacheName) const
{
	TObjectPtr<UChaosCache> const* ExistingCache = Algo::FindByPredicate(Caches, [&CacheName](const UChaosCache* Test)
	{
		if (Test)
		{
			return Test->GetFName() == CacheName;
		}
		return false;
	});

	return ExistingCache ? *ExistingCache : nullptr;
}

UChaosCache* UChaosCacheCollection::FindOrAddCache(const FName& CacheName)
{
	FName        FinalName   = CacheName;
	UChaosCache* ResultCache = nullptr;

	if(FinalName != NAME_None)
	{
		TObjectPtr<UChaosCache>* ExistingCache = Algo::FindByPredicate(Caches, [&FinalName](const UChaosCache* Test)
		{
			if (Test)
			{
				return Test->GetFName() == FinalName;
			}
			return false;
		});

		ResultCache = ExistingCache ? ExistingCache->Get() : nullptr;
	}

	if(!ResultCache)
	{
		// Final check for unique name, or no name - generate one from the base name
		// (GetPlainNameString so we don't get increasing strings of appended numbers)
		if(StaticFindObject(UChaosCache::StaticClass(), this, *FinalName.ToString()) || FinalName == NAME_None)
		{
			FinalName = MakeUniqueObjectName(this, UChaosCache::StaticClass(), *FinalName.GetPlainNameString());
		}

		ResultCache = NewObject<UChaosCache>(this, FinalName, RF_Transactional);

		Caches.Add(ResultCache);
	}

	return ResultCache;
}

void UChaosCacheCollection::FlushAllCacheWrites()
{
	ParallelFor(Caches.Num(), [this](int32 InIndex)
	{
		if (Caches[InIndex])
		{
			Caches[InIndex]->FlushPendingFrames();
		}
	});
}

float UChaosCacheCollection::GetMaxDuration() const
{
	float MaxDuration = 0.0;
    for( UChaosCache* CacheInstance : Caches)
    {
    	MaxDuration = FMath::Max(CacheInstance->GetDuration(), MaxDuration);
    }
	return MaxDuration;
}

uint32 UChaosCacheCollection::GetMaxNumFrames() const
{
	float MaxDuration = 0.0;
	uint32 MaxFramesCount = 0;
	for(const UChaosCache* CacheInstance : Caches)
	{
		if(CacheInstance->GetDuration() >= MaxDuration)
		{
			MaxDuration = CacheInstance->GetDuration();
			MaxFramesCount = CacheInstance->NumRecordedFrames;
		}
	}
	return MaxFramesCount;
}

#if WITH_EDITOR
void UChaosCacheCollection::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UChaosCacheCollection, InterpolationMode))
	{
		SetInterpolationMode(InterpolationMode);
	}
}
#endif

void UChaosCacheCollection::SetInterpolationMode(EChaosCacheInterpolationMode Mode)
{
	InterpolationMode = Mode;
	for (UChaosCache* CacheInstance : Caches)
	{
		if (CacheInstance)
		{
			CacheInstance->InterpolationMode = Mode;
		}
	}
}


