// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/CacheEvents.h"
#include "Logging/MessageLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CacheEvents)

FCacheEventTrack::FCacheEventTrack()
	: Name(NAME_None)
	, Struct(nullptr)
{

}

FCacheEventTrack::FCacheEventTrack(FName InName, UScriptStruct* InStruct)
	: Name(InName)
	, Struct(InStruct)
{

}

FCacheEventTrack::~FCacheEventTrack()
{
	DestroyAll();
}

FCacheEventTrack::FHandle FCacheEventTrack::GetEventHandle(int32 Index) const
{
	FHandle NewHandle;
	if(EventData.IsValidIndex(Index))
	{
		NewHandle.Track = this;
		NewHandle.Index = Index;
		NewHandle.Version = TransientVersion;
	}

	return NewHandle;
}

void FCacheEventTrack::DestroyAll()
{
	if(!Struct)
	{
		// Make sure we haven't lost the struct after adding some items
		check(EventData.Num() == 0);
		return;
	}

	// Invalidate old handles
	++TransientVersion;

	for(uint8* EventPtr : EventData)
	{
		Struct->DestroyStruct(EventPtr);
		FMemory::Free(EventPtr);
	}

	EventData.Reset();
	TimeStamps.Reset();
}

bool FCacheEventTrack::Serialize(FArchive& Ar)
{
	// Serialize the normal UPROPERTY data
	if(Ar.IsLoading() || Ar.IsSaving())
	{
		const UScriptStruct* ThisStruct = FCacheEventTrack::StaticStruct();
		ThisStruct->SerializeTaggedProperties(Ar, reinterpret_cast<uint8*>(this), Struct, nullptr);
	}

	if(Ar.IsSaving())
	{
		SaveEventsToArchive(Ar);
	}
	else if(Ar.IsLoading())
	{
		LoadEventsFromArchive(Ar);
	}

	return true;
}

void FCacheEventTrack::Merge(FCacheEventTrack&& Other)
{
	if(Other.TimeStamps.Num() == 0)
	{
		return;
	}

	// Invalidate old handles
	++Other.TransientVersion;
	++TransientVersion;

	const int32 NumEntries = Other.TimeStamps.Num();
	for(int32 Index = 0; Index < NumEntries; ++Index)
	{
		int32 InsertIndex = Algo::LowerBound(TimeStamps, Other.TimeStamps[Index]);

		if(TimeStamps.IsValidIndex(InsertIndex))
		{
			TimeStamps.Insert(Other.TimeStamps[Index], InsertIndex);
			EventData.Insert(Other.EventData[Index], InsertIndex);
		}
		else
		{
			TimeStamps.Add(Other.TimeStamps[Index]);
			EventData.Add(Other.EventData[Index]);
		}
	}

	Other.TimeStamps.Reset();
	Other.EventData.Reset();
}

int32 FCacheEventTrack::GetTransientVersion() const
{
	return TransientVersion;
}

void FCacheEventTrack::PushEventInternal(float TimeStep, const void* Event)
{
	// Validation already performed by FCacheEventTrack::PushEvent for the struct
	uint8* NewData = reinterpret_cast<uint8*>(FMemory::Malloc(Struct->GetStructureSize()));

	// Construct and then copy struct data
	Struct->InitializeStruct(NewData);
	Struct->CopyScriptStruct(NewData, Event);

	TimeStamps.Add(TimeStep);
	EventData.Add(NewData);

	// Invalidate old handles
	++TransientVersion;
}

void FCacheEventTrack::LoadEventsFromArchive(FArchive& Ar)
{
	UScriptStruct* StructToLoad = Struct;
	if(!StructToLoad)
	{
		// This struct contains no data so won't load garbage - but lets us continue
		StructToLoad = FCacheEventBase::StaticStruct();

		FMessageLog("LoadErrors").Error()
			->AddToken(FTextToken::Create(NSLOCTEXT("ChaosCache", "StructLoadError", "Failed to load Chaos cache event track as the underlying struct was unavailable.")));
	}

	const int32 NumEvents = TimeStamps.Num();

	for(int32 Index = 0; Index < NumEvents; ++Index)
	{
		uint8* NewEvent = reinterpret_cast<uint8*>(FMemory::Malloc(StructToLoad->GetStructureSize()));

		StructToLoad->InitializeStruct(NewEvent);
		StructToLoad->SerializeItem(Ar, NewEvent, nullptr);

		EventData.Add(NewEvent);
	}
}

void FCacheEventTrack::SaveEventsToArchive(FArchive& Ar)
{
	UScriptStruct* StructToSave = Struct;
	if(!StructToSave)
	{
		// This struct contains no data so won't load garbage - but lets us continue
		StructToSave = FCacheEventBase::StaticStruct();

		FMessageLog("SaveErrors").Error()
			->AddToken(FTextToken::Create(NSLOCTEXT("ChaosCache", "StructSaveError", "Failed to save Chaos cache event track as the underlying struct was unavailable.")));
	}

	check(EventData.Num() == TimeStamps.Num());

	for(uint8* EventPtr : EventData)
	{
		StructToSave->SerializeItem(Ar, EventPtr, nullptr);
	}
}

bool FCacheEventTrack::FHandle::IsAlive() const
{
	return Index != INDEX_NONE && Track && Track->GetTransientVersion() == Version;
}

