// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/CacheManagerActor.h"

#include "Chaos/Adapters/CacheAdapter.h"
#include "Chaos/CacheCollection.h"
#include "Chaos/CacheEvents.h"
#include "Chaos/ChaosCache.h"
#include "ChaosSolversModule.h"
#include "Components/BillboardComponent.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "Engine/Texture2D.h"
#include "PBDRigidsSolver.h"
#include "Features/IModularFeatures.h"
#include "Algo/Find.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CacheManagerActor)

#if WITH_EDITOR
#include "AssetToolsModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Editor.h"
#include "Framework/Notifications/NotificationManager.h"
#include "UObject/ConstructorHelpers.h"
#include "Widgets/Notifications/SNotificationList.h"
#else
#include "Engine/World.h"
#endif

#define LOCTEXT_NAMESPACE "ChaosCacheManager"

FName GetComponentCacheName(UPrimitiveComponent* InComponent)
{
	return FName(InComponent->GetPathName(InComponent->GetWorld()));
}

void FObservedComponent::ResetRuntimeData(const EStartMode ManagerStartMode)
{
	bTriggered       = ManagerStartMode == EStartMode::Timed;
	AbsoluteTime     = 0;
	TimeSinceTrigger = 0;
	Cache            = nullptr;

	TickRecord.Reset();
}

bool FObservedComponent::IsEnabled(ECacheMode CacheMode) const
{
	return (CacheMode == ECacheMode::Record) || bPlaybackEnabled;
}

void FObservedComponent::PostSerialize(const FArchive& Ar)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (Ar.IsLoading() && ComponentRef.OtherActor.IsValid())
	{
		SoftComponentRef.OtherActor = ComponentRef.OtherActor.Get();
		SoftComponentRef.ComponentProperty = ComponentRef.ComponentProperty;
		SoftComponentRef.OverrideComponent = ComponentRef.OverrideComponent;
		SoftComponentRef.PathToComponent = ComponentRef.PathToComponent; 
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
UPrimitiveComponent* FObservedComponent::GetComponent(AActor* OwningActor)
{
	return Cast<UPrimitiveComponent>(SoftComponentRef.GetComponent(OwningActor));
}

UPrimitiveComponent* FObservedComponent::GetComponent(AActor* OwningActor) const
{
	return Cast<UPrimitiveComponent>(SoftComponentRef.GetComponent(OwningActor));
}

AChaosCacheManager::AChaosCacheManager(const FObjectInitializer& ObjectInitializer /*= FObjectInitializer::Get()*/)
	: Super(ObjectInitializer)
	, CacheMode(ECacheMode::Record)
	, StartMode(EStartMode::Timed)
	, bStartOnBeginPlay(true)
	, bCanRecord(true)
	, bIsSimulating(false)
	, StartTimeAtBeginPlay(0)
	, PreviousStartTime(0)
{
	// This actor will tick, just not normally. There needs to be a tick-like event both before physics simulation
	// and after physics simulation, we bind to some physics scene events in BeginPlay to handle this.
	PrimaryActorTick.bCanEverTick = true;

	// Add a scene component as our root
	RootComponent = ObjectInitializer.CreateDefaultSubobject<USceneComponent>(this, TEXT("Root"));
	RootComponent->SetMobility(EComponentMobility::Static);

	// Add a sprite when in the editor
#if WITH_EDITOR
	if (!IsRunningCommandlet())
	{
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> SpriteTextureObject;
			FName                                                 ID_CacheManager;
			FText                                                 NAME_CacheManager;

			FConstructorStatics()
				: SpriteTextureObject(TEXT("/Engine/EditorResources/S_Actor"))
				, ID_CacheManager(TEXT("Cache Manager"))
				, NAME_CacheManager(NSLOCTEXT("SpriteCategory", "CacheManager", "Chaos Cache Manager"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		EditorIconComponent = ObjectInitializer.CreateEditorOnlyDefaultSubobject<UBillboardComponent>(this, TEXT("Editor Icon"));

		if (EditorIconComponent)
		{
			EditorIconComponent->Sprite = ConstructorStatics.SpriteTextureObject.Get();
			EditorIconComponent->SpriteInfo.Category = ConstructorStatics.ID_CacheManager;
			EditorIconComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_CacheManager;
			EditorIconComponent->Mobility = EComponentMobility::Static;
			EditorIconComponent->AttachToComponent(RootComponent, FAttachmentTransformRules::KeepRelativeTransform);
		}

	}
#endif
}

void AChaosCacheManager::TickActor(float DeltaTime, enum ELevelTick TickType, FActorTickFunction& ThisTickFunction)
{
	Super::TickActor(DeltaTime, TickType, ThisTickFunction);

	if(CacheCollection)
	{
		CacheCollection->FlushAllCacheWrites();
	}
}

#if WITH_EDITOR
bool
AChaosCacheManager::ContainsProperty(
	const UStruct* Struct,
	const void* InProperty) const
{
	for (FPropertyValueIterator PropIt(FProperty::StaticClass(), Struct, this); PropIt; ++PropIt)
	{
		const void* CurrProperty = PropIt.Key();
		if (InProperty == CurrProperty)
		{
			return true;
		}
	}
	return false;
}

bool AChaosCacheManager::CanEditChange(const FProperty* InProperty) const
{
	const bool bRetVal = Super::CanEditChange(InProperty);
	if (!bRetVal)
	{
		return false;
	}

	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(FObservedComponent, USDCacheDirectory))
	{
		// Find the FObservedComponent this FProperty corresponds to, and determine if the
		// associated component has an associated adapter that supports the USD workflow.

		// We hard code the type name of the component we look for, because currently the 
		// tetrahedral component is the only one supported, and it exists in ChaosFlesh, which is 
		// a downstream dependency of Chaos. Once all adapters for deformables support the USD 
		// caching workflow, then we can target a base class that exists in Chaos.
		FName TetCompName(TEXT("DeformableTetrahedralComponent"));
		for (const FObservedComponent& Observed : ObservedComponents)
		{
			TUniquePtr<FStructOnScope> StructOnScope(
				new FStructOnScope(FObservedComponent::StaticStruct(), (uint8*)&Observed));
			if (StructOnScope)
			{
				if (const UStruct* Struct = StructOnScope->GetStruct())
				{
					if (ContainsProperty(Struct, InProperty))
					{
						if (UPrimitiveComponent* Comp = Observed.GetComponent(const_cast<AChaosCacheManager*>(this)))
						{
							UClass* Class = Comp->GetClass();
							do {
								FName ClassFName = Class->GetFName();
								if (Class->GetFName() == TetCompName)
								{
									return true;
								}
								Class = Class->GetSuperClass();
							} while (Class);
						}
					}
				}
			}
		}
		// Otherwise, no.
		return false;
	}

	return bRetVal;
}
#endif

#if WITH_EDITOR

void AChaosCacheManager::PreEditChange(class FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	if (PropertyAboutToChange)
	{
		if (PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(AChaosCacheManager, ObservedComponents))
		{
			EnsureObservedComponentsNotifierRegistration();
		}
		else if (PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(AChaosCacheManager, StartTime))
		{
			PreviousStartTime = StartTime;
		}
	}
}

void AChaosCacheManager::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FName PropertyName = (PropertyChangedEvent.Property != NULL) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if (PropertyName == GET_MEMBER_NAME_CHECKED(AChaosCacheManager, CacheMode))
	{
		SetCacheMode(CacheMode);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AChaosCacheManager, StartTime)) 
	{
		// note : we cannot call SetStart time because it would override PreviousStartTime ( see also PreEditChange where PreviousStartTime is updated when the property changes )
		if (!IsRecording())
		{
			OnStartFrameChanged(StartTime);
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AChaosCacheManager, CacheCollection))
	{
		SetCacheCollection(CacheCollection);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AChaosCacheManager, ObservedComponents))
	{
		EnsureObservedComponentsNotifierRegistration();
	}

}
#endif


void AChaosCacheManager::Destroyed()
{
	Super::Destroyed();

	ClearObservedComponents(/*bDestroyCOmponents*/false);
}

void AChaosCacheManager::OnObservedComponentDestroyed(const FObservedComponent& ObservedComponent)
{
	if (UPrimitiveComponent* PrimComp = ObservedComponent.GetComponent(this))
	{
		PrimComp->BodyInstance.bSimulatePhysics = ObservedComponent.bIsSimulating;
		if (UGeometryCollectionComponent* GeomComponent = Cast<UGeometryCollectionComponent>(PrimComp))
		{
			// in record mode we need to have notify break on on the proxy 
			GeomComponent->SetNotifyBreaks(ObservedComponent.bHasNotifyBreaks);
		}
	}
}

void AChaosCacheManager::RegisterObservedComponentsNotifiers()
{
	for (FObservedComponent& ObservedComponent : ObservedComponents)
	{
		ObservedComponent.OnDestroying.BindUObject(this, &AChaosCacheManager::OnObservedComponentDestroyed);
	}

}

void AChaosCacheManager::UnregisterObservedComponentsNotifiers()
{
	for (FObservedComponent& ObservedComponent : ObservedComponents)
	{
		ObservedComponent.OnDestroying.Unbind();
	}
}

void AChaosCacheManager::EnsureObservedComponentsNotifierRegistration()
{
	UnregisterObservedComponentsNotifiers();
	RegisterObservedComponentsNotifiers();
}

void AChaosCacheManager::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	for(FObservedComponent& ObservedComponent : ObservedComponents)
	{
		ObservedComponent.PostSerialize(Ar);
		if (Ar.IsLoading() && ObservedComponent.SoftComponentRef.OtherActor == this)
		{
			// Clear OtherActor in order to use OwningActor(this) in when using GetComponent.
			// This is relevant when this CacheManager is spawnable by sequencer so the
			// SoftComponentedRef will point to the actual spawned CacheManager instance and not the spawnable template.
			ObservedComponent.SoftComponentRef.OtherActor = nullptr;
		}
	}
}

void AChaosCacheManager::SetCacheMode(ECacheMode NewCacheMode)
{
	CacheMode = NewCacheMode;
	if (IsRecording() && !CanRecord())
	{
		// If we're not allowed to record but have somehow been set to record mode, revert to default
		CacheMode = ECacheMode::None;
	}

#if WITH_EDITOR
	SetObservedComponentProperties(CacheMode);
#endif

	SetStartTime(IsRecording() ? 0 : StartTime);
}

bool AChaosCacheManager::IsRecording() const
{
	return (CacheMode == ECacheMode::Record);
}

void AChaosCacheManager::SetStartTime(float InStartTime)
{
	PreviousStartTime = StartTime;
	StartTime = InStartTime;
	if (!IsRecording())
	{
		OnStartFrameChanged(InStartTime);
	}
}

void AChaosCacheManager::SetCurrentTime(float CurrentTime)
{
	// follow to SetStartTime as this is the how we animate the cache 
	// todo(chaos) : we should probably separate the concept of start time and current time 
	SetStartTime(CurrentTime);
}

void AChaosCacheManager::ResetAllComponentTransforms()
{
	if(!CacheCollection)
	{
		return;
	}

	for(FObservedComponent& Observed : ObservedComponents)
	{
		if(UPrimitiveComponent* Comp = Observed.GetComponent(this))
		{
			if(UChaosCache* Cache = CacheCollection->FindCache(Observed.CacheName))
			{
				Comp->SetWorldTransform(Cache->Spawnable.InitialTransform);
			}
		}
	}
}

void AChaosCacheManager::ResetSingleTransform(int32 InIndex)
{
	if (!CacheCollection)
	{
		return;
	}

	if(!ObservedComponents.IsValidIndex(InIndex))
	{
		return;
	}

	FObservedComponent& Observed = ObservedComponents[InIndex];

	if(UPrimitiveComponent* Comp = Observed.GetComponent(this))
	{
		if(UChaosCache* Cache = CacheCollection->FindCache(Observed.CacheName))
		{
			Comp->SetWorldTransform(Cache->Spawnable.InitialTransform);
		}
	}
}

#if WITH_EDITOR
void AChaosCacheManager::SelectComponent(int32 InIndex)
{
	if(!ObservedComponents.IsValidIndex(InIndex))
	{
		return;
	}

	FObservedComponent& Observed = ObservedComponents[InIndex];

	if(UPrimitiveComponent* Comp = Observed.GetComponent(this))
	{
		GEditor->SelectNone(true, true);
		GEditor->SelectActor(Comp->GetOwner(), true, true);
		GEditor->SelectComponent(Comp, true, true);
	}
}
#endif


void AChaosCacheManager::Stop()
{
	EndEvaluate();
}

void AChaosCacheManager::Start(float InStartTime)
{
	Stop();

	if (IsRecording() && bCreateNewCacheCollectionAssetOnRecord)
	{
		CreateAndSetNewCacheAsset();
	}

	if (!IsRecording())
	{
		StartTime = InStartTime; // todo(ccaillaud) - is that needed in record ?
		PreviousStartTime = InStartTime;
		OnStartFrameChanged(InStartTime);
	}
	BeginEvaluate();
}

void AChaosCacheManager::BeginPlay()
{
	using namespace Chaos;

	Super::BeginPlay();

	StartTimeAtBeginPlay = StartTime;
	if (bStartOnBeginPlay)
	{
		Start(StartTime);
	}
}

void AChaosCacheManager::BeginEvaluate()
{
	EndEvaluate();

	using namespace Chaos;
	bIsSimulating = !(CacheMode == ECacheMode::None);
	
	if(!CacheCollection)
	{
		UE_LOGF(LogChaosCache, Warning, "%ls has no valid cache asset. Components will revert to dynamic simulation.", *GetName());
		
		// without a collection the cache manager can't do anything, no reason to initialise the observed array
		SetActorTickEnabled(false);
		return;
	}
	else
	{
		SetActorTickEnabled(true);
	}

	// Build list of adapters for our observed components
	IModularFeatures&               ModularFeatures = IModularFeatures::Get();
	TArray<FComponentCacheAdapter*> Adapters        = ModularFeatures.GetModularFeatureImplementations<FComponentCacheAdapter>(FComponentCacheAdapter::FeatureName);

	for (FComponentCacheAdapter* Adapter : Adapters)
	{
		Adapter->Initialize();
	}
	check(ActiveAdapters.IsEmpty()); // Should be empty from calling EndEvaluate

	int32       NumFailedPlaybackEntries = 0;
	const int32 NumComponents            = ObservedComponents.Num();
	for(int32 Index = 0; Index < NumComponents; ++Index)
	{
		FObservedComponent& Observed = ObservedComponents[Index];

		auto ByPriority = [](const FComponentCacheAdapter* A, const FComponentCacheAdapter* B) {
			return A->GetPriority() < B->GetPriority();
		};
		UPrimitiveComponent* Comp = Observed.GetComponent(this);
		
		if(!Comp)
		{
			UE_LOGF(LogChaosCache, Warning, "%ls has invalid observed component reference.", *GetName());
			ActiveAdapters.Add(nullptr);
			continue;
		}
		
		UClass* ActualClass = Comp->GetClass();
    
		TArray<FComponentCacheAdapter*> DirectAdapters = Adapters.FilterByPredicate([ActualClass](const FComponentCacheAdapter* InTest) {
			return InTest && InTest->SupportsComponentClass(ActualClass) == Chaos::FComponentCacheAdapter::SupportType::Direct;
		});

		TArray<FComponentCacheAdapter*> DerivedAdapters = Adapters.FilterByPredicate([ActualClass](const FComponentCacheAdapter* InTest) {
			return InTest && InTest->SupportsComponentClass(ActualClass) == Chaos::FComponentCacheAdapter::SupportType::Derived;
		});

		Algo::Sort(DirectAdapters, ByPriority);
		Algo::Sort(DerivedAdapters, ByPriority);

		if(DirectAdapters.Num() == 0 && DerivedAdapters.Num() == 0)
		{	
			UE_LOGF(LogChaosCache, Warning, "%ls observing component with no appropriate adapter.", *GetName());

			// No actual adapter for this class type, log and push nullptr for this observed object
			ActiveAdapters.Add(nullptr);
		}
		else
		{
			if(DirectAdapters.Num() > 0)
			{
				ActiveAdapters.Add(DirectAdapters[0]);
			}
			else
			{
				ActiveAdapters.Add(DerivedAdapters[0]);
			}
		}

		// Reset timers and last cache
		Observed.ResetRuntimeData(StartMode);

		// we need to create the adapters regardless of it being enabled for playback and we can bail if necessary after that 
		if (!Observed.IsEnabled(CacheMode))
		{
			continue;
		}

		bool                    bRequiresRecord = false;
		FComponentCacheAdapter* CurrAdapter     = ActiveAdapters[Index];
		check(CurrAdapter);    // should definitely have added one above
		if (CurrAdapter->SupportsRestartSimulation())
		{
			CurrAdapter->SetRestartSimulation(bRestartSimulation);
			CurrAdapter->SetRestartTimeRange(RestartTimeStart, RestartTimeEnd);
		}

		// We only need to register callbacks if in play/record mode
		if(CacheMode != ECacheMode::None)
		{
			if(Chaos::FPhysicsSolverEvents* Solver = CurrAdapter->BuildEventsSolver(Comp))
			{
				FPerSolverData* SolverData = PerSolverData.Find(Solver);

				if(!SolverData)
				{
					SolverData                  = &PerSolverData.Add(Solver);
					SolverData->PreSolveHandle  = Solver->AddPreAdvanceCallback(FSolverPreAdvance::FDelegate::CreateUObject(this, &AChaosCacheManager::HandlePreSolve, Solver));
					SolverData->PreBufferHandle = Solver->AddPreBufferCallback(FSolverPreAdvance::FDelegate::CreateUObject(this, &AChaosCacheManager::HandlePreBuffer, Solver));
					SolverData->PostSolveHandle = Solver->AddPostAdvanceCallback(FSolverPostAdvance::FDelegate::CreateUObject(this, &AChaosCacheManager::HandlePostSolve, Solver));
					SolverData->TeardownHandle  = Solver->AddTeardownCallback(FSolverTeardown::FDelegate::CreateUObject(this, &AChaosCacheManager::HandleTeardown, Solver));
				}

				if (CacheMode == ECacheMode::Play || !CanRecord())
				{ 
					if(UChaosCache*    PlayCache = CacheCollection->FindCache(Observed.CacheName))
					{
						FCacheUserToken Token = PlayCache->BeginPlayback();
						if(Token.IsOpen() && CurrAdapter->ValidForPlayback(Comp, PlayCache))
						{
							if (CacheMode == ECacheMode::Play)
							{
								SolverData->PlaybackIndices.Add(Index);
								SolverData->PlaybackTickRecords.AddDefaulted();
								SolverData->PlaybackTickRecords.Last().SetSpaceTransform(GetTransform());
								Observed.Cache = PlayCache;
								Observed.TickRecord.SetSpaceTransform(GetTransform());
								Observed.TickRecord.SetLastTime(StartTime);
								OpenPlaybackCaches.Add(TTuple<FCacheUserToken, UChaosCache*>(MoveTemp(Token), Observed.Cache));
								CurrAdapter->InitializeForPlayback(Comp, Observed, StartTime);
							}
						}
						else
						{
							if(Token.IsOpen())
							{
								UE_LOGF(LogChaosCache,
										Warning,
										"Failed playback for component %ls, Selected cache adapter unable to handle the cache (the cache is incompatible)",
										*Comp->GetPathName());

								// The cache session was valid so make sure to end it
								PlayCache->EndPlayback(Token);
							}
							else    // Already open for record somewhere
								{
								UE_LOGF(LogChaosCache,
										Warning,
										"Failed playback for component %ls using cache %ls, cache already open for record",
										*Comp->GetName(),
										*PlayCache->GetPathName());
								}

							++NumFailedPlaybackEntries;
						}
					}
					else
					{
						UE_LOGF(LogChaosCache, Log, "Skipping playback for component %ls, no available cache.", *Comp->GetName());
					}
				}
				else // CacheMode == ECacheMode::Record
				{
					FName CacheName = (Observed.CacheName == NAME_None) ? MakeUniqueObjectName(CacheCollection, UChaosCache::StaticClass(), "Cache") : Observed.CacheName;

					UChaosCache*    RecordCache = CacheCollection->FindOrAddCache(CacheName);
					FCacheUserToken Token       = RecordCache->BeginRecord(Comp, CurrAdapter->GetGuid(), GetTransform());

					if(Token.IsOpen())
					{
						SolverData->RecordIndices.Add(Index);

						Observed.Cache = CacheCollection->FindOrAddCache(CacheName);
				
						// We'll record the observed component in Cache Manager's local space.
						Observed.TickRecord.SetSpaceTransform(GetTransform());
						OpenRecordCaches.Add(TTuple<FCacheUserToken, UChaosCache*>(MoveTemp(Token), Observed.Cache));
						CurrAdapter->InitializeForRecord(Comp, Observed);

						// Ensure we enable the actor tick to flush out the pending record writes
						bRequiresRecord =  true;
					}
				}
			}
		}

		// If we're recording then the physics thread(s) will be filling queues on each cache of pending writes
		// which we consume on the game thread in the manager tick.
		SetActorTickEnabled(bRequiresRecord);
	}

	if(NumFailedPlaybackEntries > 0)
	{
		UE_LOGF(LogChaosCache, Warning, "Failed playback for %d components", NumFailedPlaybackEntries);

#if WITH_EDITOR
		FNotificationInfo Info(FText::Format(LOCTEXT("FailedPlaybackToast", "Failed Chaos cache playback for {0} components."), FText::AsNumber(NumFailedPlaybackEntries)));
		Info.ExpireDuration = 5.0f;
		Info.bFireAndForget = true;
		Info.Image          = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
		FSlateNotificationManager::Get().AddNotification(Info);
#endif
	}
	else
	{
		if (!IsRecording())
		{
			OnStartFrameChanged(StartTime);
		}
	}
}

void AChaosCacheManager::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Super::EndPlay(EndPlayReason);

	EndEvaluate();

	if (StartTimeAtBeginPlay != StartTime)
	{
		SetStartTime(StartTimeAtBeginPlay);
	}

#if WITH_EDITOR
	// if the cache collection was changed during PIE ( automatic asset creation upon recording for example ) 
	// make sure we update the editor actor with this new collection so it canb be replayed properly in non-PIE mode
	if (AActor* EditorWorldActor = EditorUtilities::GetEditorWorldCounterpartActor(this))
	{
		if (AChaosCacheManager* EditCachemanagerActor = Cast<AChaosCacheManager>(EditorWorldActor))
		{
			if (EditCachemanagerActor->CacheCollection != CacheCollection)
			{
				EditCachemanagerActor->SetCacheCollection(CacheCollection);
			}
		}
	}
#endif
}

void AChaosCacheManager::EndEvaluate()
{
	bIsSimulating = false;
	
	FChaosSolversModule* Module = FChaosSolversModule::GetModule();
	check(Module);

	WaitForObservedComponentSolverTasks();

	check(ActiveAdapters.IsEmpty() || ObservedComponents.Num() == ActiveAdapters.Num());

	// finalize all components
	for (int32 CompIndex = 0; CompIndex < ActiveAdapters.Num(); ++CompIndex)
	{
		if (Chaos::FComponentCacheAdapter* Adapter = ActiveAdapters[CompIndex])
		{
			FObservedComponent& Observed = ObservedComponents[CompIndex];
			Adapter->FinalizeComponent(Observed.GetComponent(this), Observed.GetChaosCache());
		}
	}

	// Build list of adapters for our observed components
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	TArray<Chaos::FComponentCacheAdapter*> Adapters = ModularFeatures.GetModularFeatureImplementations<Chaos::FComponentCacheAdapter>(Chaos::FComponentCacheAdapter::FeatureName);
	for (Chaos::FComponentCacheAdapter* Adapter : Adapters)
	{
		Adapter->Finalize();
	}

	for(TPair<Chaos::FPhysicsSolverEvents*, FPerSolverData> PerSolver : PerSolverData)
	{
		Chaos::FPhysicsSolverEvents* CurrSolver = PerSolver.Key;
		FPerSolverData&        CurrData   = PerSolver.Value;
		if(ensure(CurrSolver))
		{
			ensure(CurrSolver->RemovePostAdvanceCallback(CurrData.PostSolveHandle));
			ensure(CurrSolver->RemovePreBufferCallback(CurrData.PreBufferHandle));
			ensure(CurrSolver->RemovePreAdvanceCallback(CurrData.PreSolveHandle));
			ensure(CurrSolver->RemoveTeardownCallback(CurrData.TeardownHandle));

			CurrData.PostSolveHandle.Reset();
			CurrData.PreSolveHandle.Reset();
			CurrData.PreBufferHandle.Reset();
			CurrData.TeardownHandle.Reset();
		}
	}
	PerSolverData.Reset();
	ActiveAdapters.Reset();

	// Close any open caches as the session is complete. this will flush pending writes and post-process the cache
	for(TTuple<FCacheUserToken, UChaosCache*>& OpenCache : OpenRecordCaches)
	{
		OpenCache.Get<1>()->EndRecord(OpenCache.Get<0>());
	}
	OpenRecordCaches.Reset();

	for(TTuple<FCacheUserToken, UChaosCache*>& OpenCache : OpenPlaybackCaches)
	{
		OpenCache.Get<1>()->EndPlayback(OpenCache.Get<0>());
	}
	OpenPlaybackCaches.Reset();
}

void AChaosCacheManager::HandlePreSolve(Chaos::FReal InDt, Chaos::FPhysicsSolverEvents* InSolver)
{
	if(!CacheCollection)
	{
		return;
	}

	FPerSolverData* Data = PerSolverData.Find(InSolver);

	if(!Data)
	{
		ensureMsgf(false, TEXT("AChaosCacheManager::HandlePreSolve couldn't find a solver entry - a solver binding has leaked."));
		return;
	}

	TickObservedComponents(Data->PlaybackIndices, InDt, [this, Data](UChaosCache* InCache, FObservedComponent& Observed, Chaos::FComponentCacheAdapter* InAdapter) {
		UPrimitiveComponent* Comp = Observed.GetComponent(this);
		if(ensure(Comp))
		{
			InAdapter->Playback_PreSolve(Comp, InCache, Observed.TimeSinceTrigger, Observed.TickRecord, Data->PendingKinematicUpdates);
		}
	});
}

void AChaosCacheManager::HandlePreBuffer(Chaos::FReal InDt, Chaos::FPhysicsSolverEvents* InSolver)
{
	if(!CacheCollection)
	{
		return;
	}

	FPerSolverData* Data = PerSolverData.Find(InSolver);

	if(!Data)
	{
		ensureMsgf(false, TEXT("AChaosCacheManager::HandlePreBuffer couldn't find a solver entry - a solver binding has leaked."));
		return;
	}
	if(Data->PendingKinematicUpdates.Num() > 0)
	{
		// If the pending rigid kinematic particles are not empty we are dealing with a rigid solver
		Chaos::FPhysicsSolver* PhysicsSolver = StaticCast<Chaos::FPhysicsSolver*>(InSolver);
		for(Chaos::TPBDRigidParticleHandle<Chaos::FReal, 3>* PendingKinematic : Data->PendingKinematicUpdates)
		{
			PhysicsSolver->GetParticles().MarkTransientDirtyParticle(PendingKinematic);
		}
		Data->PendingKinematicUpdates.Reset();
	}
}

void AChaosCacheManager::HandlePostSolve(Chaos::FReal InDt, Chaos::FPhysicsSolverEvents* InSolver)
{
	if(!CacheCollection)
	{
		return;
	}

	FPerSolverData* Data = PerSolverData.Find(InSolver);

	if(!Data)
	{
		ensureMsgf(false, TEXT("AChaosCacheManager::HandlePostSolve couldn't find a solver entry - a solver binding has leaked."));
		return;
	}

	TickObservedComponents(Data->RecordIndices, InDt, [this](UChaosCache* InCache, FObservedComponent& Observed, Chaos::FComponentCacheAdapter* InAdapter) {
		UPrimitiveComponent* Comp = Observed.GetComponent(this);
		if (ensure(Comp && InCache))
		{
			
			// If we haven't advanced since the last record, don't push another frame
			if (Observed.TimeSinceTrigger > InCache->GetDuration())
			{
				FPendingFrameWrite NewFrame;
				float ActualTime = Observed.TimeSinceTrigger;
				if (bRestartSimulation)
				{
					ActualTime += RestartTimeStart;
				}
				NewFrame.Time = ActualTime;
				InAdapter->Record_PostSolve(Comp, Observed.TickRecord.GetSpaceTransform(), NewFrame, ActualTime);

				InCache->AddFrame_Concurrent(MoveTemp(NewFrame));
			}
		}
	});
}

void AChaosCacheManager::HandleTeardown(Chaos::FPhysicsSolverEvents* InSolver)
{
	// Solver has been deleted, we remove all reference to it to avoid dereferencing a dangling pointer.
	PerSolverData.Remove(InSolver);
}

void AChaosCacheManager::OnStartFrameChanged(Chaos::FReal InTime)
{
	if (bIsSimulating)
	{
		return;
	}
	
	if (!CacheCollection)
	{
		return;
	}
	for (int32 ObservedIdx = 0; ObservedIdx < ObservedComponents.Num(); ++ObservedIdx)
	{
		FObservedComponent& Observed = ObservedComponents[ObservedIdx];
		if (UPrimitiveComponent* Comp = Observed.GetComponent(this))
		{
			if (!Observed.BestFitAdapter)
			{
				// Cache the best fit adapater so that we don't need to look it up each time we evaluate a start time.
				Observed.BestFitAdapter = Chaos::FAdapterUtil::GetBestAdapterForClass(Comp->GetClass(), false);
			}

			if (!Observed.Cache)
			{
				Observed.Cache = CacheCollection->FindCache(Observed.CacheName);
			}

			if (!Observed.Cache || !Observed.BestFitAdapter || !Observed.IsEnabled(CacheMode))
			{
				continue;
			}

			FCacheUserToken Token = Observed.Cache->BeginPlayback();
			if (Token.IsOpen() || Observed.BestFitAdapter->SupportsPlayWhileRecording())
			{
				if (Observed.BestFitAdapter->ValidForPlayback(Comp, Observed.Cache))
				{
					// the init for load is quite lightweight and should be done before each update of time
					// @todo maybe rename SetRestState to LoadCacheAtTime
					Observed.BestFitAdapter->InitializeForLoad(Comp, Observed);
					Observed.BestFitAdapter->SetRestState(Comp, Observed.Cache, GetTransform(), InTime);

					if (bReplayEventsInStaticPoseMode && CacheMode == ECacheMode::None)
					{
						FPlaybackTickRecord TickRecord;
						TickRecord.SetLastTime(InTime);
						TickRecord.SetDt(FMath::Max(0, InTime - PreviousStartTime));
						Observed.BestFitAdapter->EmitEvents(Comp, Observed.Cache, TickRecord);
					}
				}
				if (Token.IsOpen())
				{
					Observed.Cache->EndPlayback(Token);
				}
			}
		}
	}
}

void AChaosCacheManager::ReadRestartData()
{
	if (!CacheCollection)
	{
		return;
	}
	for (int32 ObservedIdx = 0; ObservedIdx < ObservedComponents.Num(); ++ObservedIdx)
	{
		FObservedComponent& Observed = ObservedComponents[ObservedIdx];
		if (UPrimitiveComponent* Comp = Observed.GetComponent(this))
		{
			if (!Observed.BestFitAdapter)
			{
				// Cache the best fit adapater so that we don't need to look it up each time we evaluate a start time.
				Observed.BestFitAdapter = Chaos::FAdapterUtil::GetBestAdapterForClass(Comp->GetClass(), false);
			}

			if (!Observed.Cache)
			{
				Observed.Cache = CacheCollection->FindCache(Observed.CacheName);
			}

			if (!Observed.Cache || !Observed.BestFitAdapter)
			{
				continue;
			}

			FCacheUserToken Token = Observed.Cache->BeginPlayback();
			if (Token.IsOpen() || Observed.BestFitAdapter->SupportsPlayWhileRecording())
			{
				if (Observed.BestFitAdapter->ValidForPlayback(Comp, Observed.Cache))
				{
					Observed.BestFitAdapter->InitializeForLoad(Comp, Observed);
					Observed.BestFitAdapter->SetRestState(Comp, Observed.Cache, GetTransform(), RestartTimeStart);
				}
				if (Token.IsOpen())
				{
					Observed.Cache->EndPlayback(Token);
				}
			}
		}
	}
}

void AChaosCacheManager::SetCacheCollection(UChaosCacheCollection* InCacheCollection)
{
	// check if we have any observed component already triggered
	for (FObservedComponent& Observed : ObservedComponents)
	{
		if (Observed.bTriggered)
		{
			UE_LOGF(LogChaosCache, Warning, "Trying to change the cache collection while some observed component are already been triggered ( %ls )", *GetName());
			return;
		}
	}

	CacheCollection = InCacheCollection;

	// Reset cache collection
	for (FObservedComponent& Observed : ObservedComponents)
	{
		if (Observed.Cache && Observed.Cache->GetOuter() != CacheCollection)
		{
			Observed.Cache = nullptr;
		}
	}

	// reupdate the pose based on the new cache 
	SetStartTime(0);
}

void AChaosCacheManager::CreateAndSetNewCacheAsset()
{
#if WITH_EDITOR
	if (CacheCollection)
	{
		FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");

		FString NewPackageName;
		FString NewAssetName;

		static const FString EmptyDefaultSuffix;
		AssetToolsModule.Get().CreateUniqueAssetName(CacheCollection->GetOutermost()->GetName(), EmptyDefaultSuffix, /*out*/ NewPackageName, /*out*/ NewAssetName);
		const FString NewPackagePath = FPackageName::GetLongPackagePath(NewPackageName);

		UPackage* NewPackage = CreatePackage(*NewPackageName);

		EObjectFlags Flags = EObjectFlags::RF_Public | EObjectFlags::RF_Standalone;
		UChaosCacheCollection* NewCacheCollection = NewObject<UChaosCacheCollection>(NewPackage, *NewAssetName, Flags);

		NewCacheCollection->MarkPackageDirty();
		FAssetRegistryModule::AssetCreated(NewCacheCollection);

		//UObject* NewObject = AssetToolsModule.Get().CreateAsset(NewAssetName, NewPackagePath, UChaosCacheCollection::StaticClass(), /* Factory */ nullptr);
		CacheCollection = NewCacheCollection;
	}
#else
	UE_LOGF(LogChaosCache, Error, "Cache Manager [%ls] - Creating new asset upon recording is not supported in non-editor builds", *GetName());
#endif
}

void AChaosCacheManager::TriggerComponent(UPrimitiveComponent* InComponent)
{
	// #BGTODO Maybe not totally thread-safe, probably safer with an atomic or condition var rather than the bTriggered flag
	FObservedComponent* Found = Algo::FindByPredicate(ObservedComponents, [this, InComponent](const FObservedComponent& Test) {
		return Test.GetComponent(this) == InComponent && Test.IsEnabled(CacheMode);
	});

	if (Found && StartMode == EStartMode::Triggered)
	{
		Found->bTriggered = true;
	}
}

void AChaosCacheManager::TriggerComponentByCache(FName InCacheName)
{
	FObservedComponent* Found = Algo::FindByPredicate(ObservedComponents, [this, InCacheName](const FObservedComponent& Test) {
		return Test.CacheName == InCacheName && Test.GetComponent(this) && Test.IsEnabled(CacheMode);
	});

	if (Found && StartMode == EStartMode::Triggered)
	{
		Found->bTriggered = true;
	}
}

void AChaosCacheManager::TriggerAll()
{
	for(FObservedComponent& Observed : ObservedComponents)
	{
		if (StartMode == EStartMode::Triggered && Observed.GetComponent(this) && Observed.IsEnabled(CacheMode))
		{
			Observed.bTriggered = true;
		}
	}
}

void AChaosCacheManager::EnablePlaybackByCache(FName InCacheName, bool bEnable)
{
	FObservedComponent* Found = Algo::FindByPredicate(ObservedComponents, [this, InCacheName](const FObservedComponent& Test) {
		return Test.CacheName == InCacheName;
		});

	if (Found)
	{
		Found->bPlaybackEnabled = bEnable;
	}
}


void AChaosCacheManager::EnablePlayback(int32 Index, bool bEnable)
{
	if (ObservedComponents.IsValidIndex(Index))
	{
		ObservedComponents[Index].bPlaybackEnabled = bEnable;
	}
}

FObservedComponent* AChaosCacheManager::FindObservedComponent(UPrimitiveComponent* InComponent)
{
	return Algo::FindByPredicate(ObservedComponents, [this, ToTest = InComponent](const FObservedComponent& Item) {
		return Item.GetComponent(this) == ToTest;
	});
}

FObservedComponent& AChaosCacheManager::AddNewObservedComponent(UPrimitiveComponent* InComponent)
{
	// this check is preventing components being spawned from CS
	//check(InComponent->CreationMethod != EComponentCreationMethod::UserConstructionScript);
	ObservedComponents.AddDefaulted();
	FObservedComponent& NewEntry = ObservedComponents.Last();

	AActor* const ComponentOwner = InComponent->GetOwner();
	NewEntry.SoftComponentRef.PathToComponent = InComponent->GetPathName(ComponentOwner);
	NewEntry.SoftComponentRef.OtherActor      = ComponentOwner == this ? nullptr : ComponentOwner;

	FName CacheName = NAME_None;
#if WITH_EDITOR
	CacheName = FName(InComponent->GetOwner()->GetActorLabel());
#endif

	NewEntry.CacheName = MakeUniqueObjectName(CacheCollection, UChaosCache::StaticClass(), CacheName);

	// make sure we keep track of the various flag that may be changed by the cahe to function
	NewEntry.bIsSimulating = InComponent->IsSimulatingPhysics();
	NewEntry.bHasNotifyBreaks = false;
	if (UGeometryCollectionComponent* GeomComponent = Cast<UGeometryCollectionComponent>(InComponent))
	{
		NewEntry.bHasNotifyBreaks = GeomComponent->bNotifyBreaks;
	}

	EnsureObservedComponentsNotifierRegistration();

	return NewEntry;
}

void AChaosCacheManager::FindOrAddObservedComponent(UPrimitiveComponent* InComponent, const FName& CacheName, const bool bTransferSimulation)
{
	if(InComponent != nullptr)
	{
		// we do not want to record transient component as they may not exists when we start recording 
		if (InComponent->HasAnyFlags(RF_Transient))
		{
			return;
		}
		FObservedComponent* FoundComponent = FindObservedComponent(InComponent);
		if(FObservedComponent* ObservedComponent = FoundComponent ? FoundComponent : &AddNewObservedComponent(InComponent))
		{
			ObservedComponent->bIsSimulating = bTransferSimulation ? InComponent->BodyInstance.bSimulatePhysics : false;
			if (UGeometryCollectionComponent* GeomComponent = Cast<UGeometryCollectionComponent>(InComponent))
			{
				ObservedComponent->bHasNotifyBreaks = GeomComponent->bNotifyBreaks;
			}

			if(CacheName != TEXT("")) 
			{
				ObservedComponent->CacheName = CacheName;
			}
		}
	}
}

void AChaosCacheManager::RemoveObservedComponent(UPrimitiveComponent* PrimitiveComponent)
{
	EnsureObservedComponentsNotifierRegistration();
	// make sure we listen to those component being removed
	ObservedComponents.RemoveAll([this, ToTest = PrimitiveComponent](const FObservedComponent& Item)
	{
		return Item.GetComponent(this) == ToTest;
	});
}

void AChaosCacheManager::ClearObservedComponents(bool bDestroyComponents)
{
	if (bDestroyComponents)
	{
		// we are destroying, no need to restore state
		UnregisterObservedComponentsNotifiers();
		for (FObservedComponent& ObservedComponent : ObservedComponents)
		{
			UPrimitiveComponent* Comp = ObservedComponent.GetComponent(this);
			Comp->DestroyComponent();
		}
	}
	EnsureObservedComponentsNotifierRegistration();
	ObservedComponents.Empty();
}

void AChaosCacheManager::TickObservedComponents(const TArray<int32>& InIndices, Chaos::FReal InDt, FTickObservedFunction InCallable)
{
	for(int32 Index : InIndices)
	{
		check(ObservedComponents.IsValidIndex(Index) && ObservedComponents.Num() == ActiveAdapters.Num());

		FObservedComponent&            Observed = ObservedComponents[Index];
		Chaos::FComponentCacheAdapter* Adapter  = ActiveAdapters[Index];

		if(!Observed.Cache || !Observed.IsEnabled(CacheMode))
		{
			// Skip if no available cache - this can happen if a component was deleted while being observed - the other components
			// can play fine, we just omit any that we cannot find.
			continue;
		}

		Observed.AbsoluteTime += InDt;

		// Adapters can be null if there isn't support available for a selected component
		// (happens if a plugin implemented it but is no longer loaded)
		if(Observed.bTriggered && Adapter)
		{
			if (CacheMode == ECacheMode::Play)
			{
				Observed.TickRecord.SetDt(InDt);
			}

			Observed.TimeSinceTrigger += InDt;
			InCallable(Observed.Cache, Observed, Adapter);
		}
	}
}

void AChaosCacheManager::WaitForObservedComponentSolverTasks()
{
	check(ActiveAdapters.IsEmpty() || ObservedComponents.Num() == ActiveAdapters.Num());

	for (int32 CompIndex = 0; CompIndex < ActiveAdapters.Num(); ++CompIndex)
	{
		if(const Chaos::FComponentCacheAdapter* Adapter = ActiveAdapters[CompIndex])
		{
			FObservedComponent& Observed = ObservedComponents[CompIndex];
			Adapter->WaitForSolverTasks(Observed.GetComponent(this));
		}
	}
}

#if WITH_EDITOR
void AChaosCacheManager::SetObservedComponentProperties(const ECacheMode& NewCacheMode)
{
	for (FObservedComponent& ObservedComponent : ObservedComponents)
	{
		if (UPrimitiveComponent* PrimComp = ObservedComponent.GetComponent(this))
		{
			if (NewCacheMode == ECacheMode::Record)
			{
				PrimComp->BodyInstance.bSimulatePhysics = ObservedComponent.bIsSimulating;
				if (UGeometryCollectionComponent* GeomComponent = Cast<UGeometryCollectionComponent>(PrimComp))
				{
					// in record mode we need to have notify break on on the proxy 
					GeomComponent->SetNotifyBreaks(true);
				}
			}
			else
			{
				PrimComp->BodyInstance.bSimulatePhysics = false;
				if (UGeometryCollectionComponent* GeomComponent = Cast<UGeometryCollectionComponent>(PrimComp))
				{
					// in playback modes we can restore the state of the notification 
					GeomComponent->SetNotifyBreaks(ObservedComponent.bHasNotifyBreaks);
				}
			}
		}
	}
}

void AChaosCacheManager::SetEditorIconVisibility(bool bVisibility)
{
	if (EditorIconComponent)
	{
		EditorIconComponent->SetVisibility(bVisibility);
	}
}

#endif	// WITH_EDITOR

AChaosCachePlayer::AChaosCachePlayer(const FObjectInitializer& ObjectInitializer /*= FObjectInitializer::Get()*/)
	: Super(ObjectInitializer)
{
	bCanRecord = false;
}

#undef LOCTEXT_NAMESPACE

