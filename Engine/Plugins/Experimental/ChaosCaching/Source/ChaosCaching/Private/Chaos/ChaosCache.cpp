// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/ChaosCache.h"
#include "Chaos/CacheEvents.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/TransformCollection.h"
#include "UObject/Package.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "Math/DualQuat.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosCache)

bool bChaosCacheUseInterpolation = true;
FAutoConsoleVariableRef CVarChaosCacheUseInterpolation(
	TEXT("p.Chaos.Cache.UseInterpolation"),
	bChaosCacheUseInterpolation,
	TEXT("When enabled, cache interpolates between keys.[def: true]"));

bool bChaosCacheCompressTracksAfterRecording = true;
FAutoConsoleVariableRef CVarChaosCacheCompressTracksAfterRecording(
	TEXT("p.Chaos.Cache.CompressTracksAfterRecording"),
	bChaosCacheCompressTracksAfterRecording,
	TEXT("When enabled, cache will compress the transform tracks after recording is done.[def: true]"));


UChaosCache::UChaosCache()
	: CurrentRecordCount(0)
	, CurrentPlaybackCount(0)
	, bStripMassToLocal(false)
{
	// Default Version. If a Version was archived, this will be overwritten during Serialize() 
	Version = 0;
}

void UChaosCache::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{ 
		// Older versions of GeometryCollection caches had MassToLocal transform baked into the stored transforms. This unfortunately means
		// that evaluating the cache outside the context of the PhysicsThread is unlikely to be accurate. To make this work, we need to
		// strip the MassToLocal from the existing cached transforms.
		if ((Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::GeometryCollectionCacheRemovesMassToLocal) && (Version == 0))
		{
			bStripMassToLocal = true;
		}
	}	
}

void UChaosCache::PostLoad()
{
	if (bStripMassToLocal)
	{ 
		// Am I a GeometryCollection cache?
		UObject* DuplicatedTemplate = Spawnable.DuplicatedTemplate;
		if (UGeometryCollectionComponent* GeometryCollectionComponent = Cast<UGeometryCollectionComponent>(DuplicatedTemplate))
		{
			// Get the referenced RestCollection
			if (const UGeometryCollection* RestCollection = GeometryCollectionComponent->GetRestCollection())
			{
				const TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollection = RestCollection->GetGeometryCollection();

				// Get the MassToLocal transforms
				if (GeometryCollection->HasAttribute(TEXT("MassToLocal"), FTransformCollection::TransformGroup))
				{
					const TManagedArray<FTransform>& CollectionMassToLocal = GeometryCollection->GetAttribute<FTransform>(TEXT("MassToLocal"), FTransformCollection::TransformGroup);
				
					// Strip out the MassToLocal transforms from all cached transforms
					int32 NumTracks = TrackToParticle.Num();
					int32 NumParticles = CollectionMassToLocal.Num();
					for (int32 TrackIdx = 0; TrackIdx < NumTracks; ++TrackIdx)
					{
						int32 ParticleIdx = TrackToParticle[TrackIdx];
						if (ParticleIdx < NumParticles)
						{ 
							const FTransform& MassToLocal = CollectionMassToLocal[ParticleIdx];
							FTransform MassToLocalInverse = MassToLocal.Inverse();
							FTransform3f MassToLocalInversef(FQuat4f(MassToLocalInverse.GetRotation()), FVector3f(MassToLocalInverse.GetTranslation()), FVector3f(MassToLocalInverse.GetScale3D()));

							FParticleTransformTrack& TransformData = ParticleTracks[TrackIdx].TransformData;
							FRawAnimSequenceTrack& AnimTrack = TransformData.RawTransformTrack;

							if (ensure((AnimTrack.PosKeys.Num() == AnimTrack.RotKeys.Num()) && (AnimTrack.RotKeys.Num() == AnimTrack.ScaleKeys.Num())))
							{ 
								int32 NumKeys = AnimTrack.PosKeys.Num();
								for (int32 KeyIdx = 0; KeyIdx < NumKeys; ++KeyIdx)
								{
									FTransform3f MassTransform(AnimTrack.RotKeys[KeyIdx], AnimTrack.PosKeys[KeyIdx], AnimTrack.ScaleKeys[KeyIdx]);
									FTransform3f LocalTransform = MassToLocalInversef * MassTransform;
									const FQuat4f& LocalRotation = LocalTransform.GetRotation();
									const FVector3f& LocalTranslation = LocalTransform.GetTranslation();

									AnimTrack.RotKeys[KeyIdx] = LocalRotation;
									AnimTrack.PosKeys[KeyIdx] = LocalTranslation;
								}
							}
						}
					}
				
					bStripMassToLocal = false;
										
				}
			}
		}
	}

	// Up-to-date now
	Version = CurrentVersion;

	// Build ParticleToChannelCurve 
	ParticleToChannelCurve.Reserve(ChannelCurveToParticle.Num());
	for (int32 ChannelCurveIndex = 0; ChannelCurveIndex < ChannelCurveToParticle.Num(); ++ChannelCurveIndex)
	{
		ParticleToChannelCurve.Add(ChannelCurveToParticle[ChannelCurveIndex], ChannelCurveIndex);
	}

	Super::PostLoad();
}

void UChaosCache::FlushPendingFrames_ChannelOnlyReservePass(TQueue<FPendingFrameWrite, EQueueMode::Spsc>& LocalPendingWrites,
	bool& bCanSimpleCopyChannelData)
{
	bCanSimpleCopyChannelData = true;

	TMap<FName, int32> ChannelKeys;
	FPendingFrameWrite NewData;
	while (PendingWrites.Dequeue(NewData))
	{
		QUICK_SCOPE_CYCLE_COUNTER(QSTAT_CacheFlushSingleFrameReservePass);

		const bool bContainsParticleData = NewData.PendingParticleData.Num() > 0;
		if (!bContainsParticleData)
		{
			const int32 NumParticles = NewData.PendingChannelsIndices.Num();
			for (const TPair<FName, TArray<float>>& ChannelData : NewData.PendingChannelsData)
			{
				int32* ChannelKeyCount = ChannelKeys.Find(ChannelData.Key);
				if (!ChannelKeyCount)
				{
					ChannelKeyCount = &ChannelKeys.Add(ChannelData.Key);
					*ChannelKeyCount = 0;
				}
				(*ChannelKeyCount)++;
			}

			for (int32 PendingIndex = 0; PendingIndex < NumParticles; ++PendingIndex)
			{
				const int32 ParticleIndex = NewData.PendingChannelsIndices[PendingIndex];
				if (bCanSimpleCopyChannelData && PendingIndex < ChannelCurveToParticle.Num())
				{
					bCanSimpleCopyChannelData = ChannelCurveToParticle[PendingIndex] == ParticleIndex;
				}

				if (!ParticleToChannelCurve.Contains(ParticleIndex))
				{
					checkSlow(!ChannelCurveToParticle.Contains(ParticleIndex));
					const int32 ChannelIndex = ChannelCurveToParticle.Add(ParticleIndex);
					ParticleToChannelCurve.Add(ParticleIndex, ChannelIndex);
				}
			}
		}

		LocalPendingWrites.Enqueue(MoveTemp(NewData));
	}

	const int32 NumChannels = ChannelCurveToParticle.Num();
	for (const TPair<FName, int32>& KeyCount : ChannelKeys)
	{
		FRichCurves* TargetCurve = ChannelsTracks.Find(KeyCount.Key);
		if (TargetCurve == nullptr)
		{
			TargetCurve = &ChannelsTracks.Add(KeyCount.Key);
			TargetCurve->RichCurves.SetNum(NumChannels);
		}
		for (FRichCurve& Curve : TargetCurve->RichCurves)
		{
			Curve.ReserveKeys(Curve.GetNumKeys() + KeyCount.Value);
		}
	}
}

template<EQueueMode Mode>
bool UChaosCache::FlushPendingFrames_MainPass(TQueue<FPendingFrameWrite, Mode>& InPendingWrites, bool bCanSimpleCopyChannelData)
{
	bool bWroteParticleData = false;
	FPendingFrameWrite NewData;

	auto WriteDataToTransformTrack = [](FParticleTransformTrack& PTrack, const float Time, const bool bPendingDeactivate, const FTransform& PendingTransform) -> bool
	{
		if (PTrack.GetNumKeys() == 0)
		{
			// Initial write to this particle
			PTrack.BeginOffset = Time;

			// Particle will hold at end of recording.
			PTrack.bDeactivateOnEnd = false;
		}

		if (bPendingDeactivate)
		{
			// Signals that this is the final keyframe and that the particle then deactivates.
			PTrack.bDeactivateOnEnd = true;
		}

		if (ensure(PTrack.GetNumKeys() == 0 || Time > PTrack.KeyTimestamps.Last()))
		{
			PTrack.KeyTimestamps.Add(Time);

			// Append the transform (ignoring scale)
			FRawAnimSequenceTrack& RawTrack = PTrack.RawTransformTrack;
			RawTrack.ScaleKeys.Add(FVector3f(1.0f));
			RawTrack.PosKeys.Add(FVector3f(PendingTransform.GetTranslation()));
			RawTrack.RotKeys.Add(FQuat4f(PendingTransform.GetRotation()));
			return true;
		}
		return false;
	};
	
	while (InPendingWrites.Dequeue(NewData))
	{
		QUICK_SCOPE_CYCLE_COUNTER(QSTAT_CacheFlushSingleFramePass2);
		{
			const bool bContainsParticleData = NewData.PendingParticleData.Num() > 0;
			const int32 NumParticles = bContainsParticleData ? NewData.PendingParticleData.Num() : NewData.PendingChannelsIndices.Num();
			bWroteParticleData |= NumParticles > 0;
			for (auto& ChannelData : NewData.PendingChannelsData)
			{
				// TArray<FRichCurve>& TargetCurve = ChannelsTracks.FindOrAdd(ChannelData.Key).RichCurves;
				// TargetCurve.SetNum(NumParticles);

				FRichCurves* TargetCurve = ChannelsTracks.Find(ChannelData.Key);
				if (TargetCurve == nullptr)
				{
					TargetCurve = &ChannelsTracks.Add(ChannelData.Key);
					TargetCurve->RichCurves.SetNum(NumParticles);
				}

				// If there is no particle data, use ChannelCurveToParticle instead.				
				check(ChannelData.Value.Num() <= NumParticles);
				if (bContainsParticleData)
				{
					for (int32 PendingIndex = 0; PendingIndex < NumParticles; ++PendingIndex)
					{
						const FPendingParticleWrite& ParticleData = NewData.PendingParticleData[PendingIndex];
						const int32 ParticleIndex = ParticleData.ParticleIndex;

						int32 TrackIndex = INDEX_NONE;
						if (!TrackToParticle.Find(ParticleIndex, TrackIndex))
						{
							TrackToParticle.Add(ParticleIndex);
							TrackIndex = ParticleTracks.AddDefaulted();
						}
						TargetCurve->RichCurves[TrackIndex].AddKey(NewData.Time, ChannelData.Value[PendingIndex]);
					}
				}
				else if (bCanSimpleCopyChannelData)
				{
					int32 ParticleIndex = 0;
					for (auto& ChannelValue : ChannelData.Value)
					{
						TargetCurve->RichCurves[ParticleIndex++].AddKey(NewData.Time, ChannelValue);
					}
				}
				else
				{
					for (int32 PendingIndex = 0; PendingIndex < NumParticles; ++PendingIndex)
					{
						const int32 ParticleIndex = NewData.PendingChannelsIndices[PendingIndex];
						int32* ChannelIndex = ParticleToChannelCurve.Find(ParticleIndex);
						if (!ChannelIndex)
						{
							ChannelIndex = &ParticleToChannelCurve.Add(ParticleIndex, ChannelCurveToParticle.Add(ParticleIndex));
						}
						if (TargetCurve->RichCurves.IsValidIndex(*ChannelIndex))
						{
							TargetCurve->RichCurves[*ChannelIndex].AddKey(NewData.Time, ChannelData.Value[PendingIndex]);
						}
					}
				}
			}
		}
		MinTime = FMath::Min(MinTime,NewData.Time);
		MaxTime = FMath::Max(MaxTime,NewData.Time);

		const int32 ParticleCount = NewData.PendingParticleData.Num();

		bWroteParticleData |= ParticleCount > 0;

		for (int32 Index = 0; Index < ParticleCount; ++Index)
		{
			const FPendingParticleWrite& ParticleData = NewData.PendingParticleData[Index];
			const int32 ParticleIndex = ParticleData.ParticleIndex;

			int32 TrackIndex = INDEX_NONE;
			if (!TrackToParticle.Find(ParticleIndex, TrackIndex))
			{
				TrackToParticle.Add(ParticleIndex);
				TrackIndex = ParticleTracks.AddDefaulted();
			}

			FPerParticleCacheData& TargetCacheData = ParticleTracks[TrackIndex];
			FParticleTransformTrack& PTrack = TargetCacheData.TransformData;

			if (WriteDataToTransformTrack(PTrack, NewData.Time, ParticleData.bPendingDeactivate, ParticleData.PendingTransform))
			{
				for (TPair<FName, float> CurveKeyPair : ParticleData.PendingCurveData)
				{
					FRichCurve& TargetCurve = TargetCacheData.CurveData.FindOrAdd(CurveKeyPair.Key);
					TargetCurve.AddKey(NewData.Time, CurveKeyPair.Value);
				}
			}
		}

		for (TTuple<FName, FCacheEventTrack>& PendingTrack : NewData.PendingEvents)
		{
			FCacheEventTrack* CacheTrack = EventTracks.Find(PendingTrack.Key);

			if (!CacheTrack)
			{
				CacheTrack = &EventTracks.Add(PendingTrack.Key, FCacheEventTrack(PendingTrack.Key, PendingTrack.Value.Struct));
			}

			CacheTrack->Merge(MoveTemp(PendingTrack.Value));
		}

		for (const TTuple<FName, FTransform>& PendingNamedTransform : NewData.PendingNamedTransformData)
		{
			FNamedTransformTrack* TransformCacheTrack = NamedTransformTracks.Find(PendingNamedTransform.Key);
			if (!TransformCacheTrack)
			{
				TransformCacheTrack = &NamedTransformTracks.Add(PendingNamedTransform.Key, FNamedTransformTrack());
			}
			WriteDataToTransformTrack(*TransformCacheTrack, NewData.Time, false, PendingNamedTransform.Value);
		}

		++NumRecordedFrames;
	}
	return bWroteParticleData;
}

void UChaosCache::FlushPendingFrames()
{
	QUICK_SCOPE_CYCLE_COUNTER(QSTAT_CacheFlushPendingFrames);
	bool bWroteParticleData = false;

	const bool bDoChannelOnlyTwoPass = PendingWrites.Peek() && PendingWrites.Peek()->PendingParticleData.Num() == 0 && PendingWrites.Peek()->PendingChannelsIndices.Num() > 0;
	if (bDoChannelOnlyTwoPass)
	{
		bool bCanSimpleCopyChannelData = false;
		TQueue<FPendingFrameWrite, EQueueMode::Spsc> LocalPendingWrites;
		FlushPendingFrames_ChannelOnlyReservePass(LocalPendingWrites, bCanSimpleCopyChannelData);
		bWroteParticleData = FlushPendingFrames_MainPass(LocalPendingWrites, bCanSimpleCopyChannelData);
	}
	else
	{
		constexpr bool bCanSimpleCopyChannelData = false;
		bWroteParticleData = FlushPendingFrames_MainPass(PendingWrites, bCanSimpleCopyChannelData);
	}

	if(bWroteParticleData)
	{
		QUICK_SCOPE_CYCLE_COUNTER(QSTAT_CacheCalcDuration);
		float Min = TNumericLimits<float>::Max();
		float Max = -Min;
		for(const FPerParticleCacheData& ParticleData : ParticleTracks)
		{
			Min = FMath::Min(Min, ParticleData.TransformData.GetBeginTime());
			Max = FMath::Max(Max, ParticleData.TransformData.GetEndTime());
		}

		for(const TPair<FName, FRichCurves>& Pair : ChannelsTracks)
		{
			const FRichCurves& RichCurves = Pair.Value;
			if (RichCurves.RichCurves.Num() > 0)
			{
				// assuming all curves have the same time range
				const FRichCurve& Curve = RichCurves.RichCurves[0];
				float CurveMin, CurveMax;
				Curve.GetTimeRange(CurveMin, CurveMax);
				Min = FMath::Min(Min, CurveMin);
				Max = FMath::Max(Max, CurveMax);
			}
		}

		RecordedDuration =  Max - Min;
	}
	// In case we are not recording to channels/particles/curves datas (USD) we can
	// use the pending writes time to figure out the recorded duration.
	// Should probably be the unique way of computing it. 
	if(ParticleTracks.IsEmpty() && ChannelsTracks.IsEmpty() && (MaxTime > MinTime))
	{
		RecordedDuration = MaxTime-MinTime;
	}
}

FCacheUserToken UChaosCache::BeginRecord(const UPrimitiveComponent* InComponent, FGuid InAdapterId, const FTransform& SpaceTransform)
{
	// First make sure we're valid to record
	int32 OtherRecordersCount = CurrentRecordCount.fetch_add(1);
	if(OtherRecordersCount == 0)
	{
		// We're the only recorder
		if(CurrentPlaybackCount.load() == 0)
		{
			// And there's no playbacks, we can proceed
			// Setup the cache to begin recording
			RecordedDuration = 0.0f;
			MinTime = TNumericLimits<float>::Max();
			MaxTime = TNumericLimits<float>::Lowest();
			NumRecordedFrames = 0;
			ParticleTracks.Reset();
			ChannelsTracks.Reset();
			TrackToParticle.Reset();
			ChannelCurveToParticle.Reset();
			CurveData.Reset();
			EventTracks.Reset();
			NamedTransformTracks.Reset();
			ParticleToChannelCurve.Reset();

			PendingWrites.Empty();

			// Initialise the spawnable template to handle the provided component
			BuildSpawnableFromComponent(InComponent, SpaceTransform);

			// Build a compatibility hash for the component state.
			AdapterGuid = InAdapterId;

			if (UPackage* Package = GetOutermost())
			{
				Package->SetDirtyFlag(true);
			}

			return FCacheUserToken(true, true, this);
		}
		else
		{
			UE_LOGF(LogChaosCache, Warning, "Failed to open cache %ls for record, it was the only recorder but the cache was open for playback.", *GetPathName());
			CurrentRecordCount--;
		}
	}
	else
	{
		UE_LOGF(LogChaosCache, Warning, "Failed to open cache %ls for record, the cache was already open for record.", *GetPathName());
		CurrentRecordCount--;
	}

	return FCacheUserToken(false, true, this);
}

void UChaosCache::EndRecord(FCacheUserToken& InOutToken)
{
	if(InOutToken.IsOpen() && InOutToken.Owner == this)
	{
		FlushPendingFrames();
		// Cache now complete, process data

		// Invalidate the token
		InOutToken.bIsOpen = false;
		InOutToken.Owner = nullptr;
		CurrentRecordCount--;

		// Set the Version appropriately
		Version = CurrentVersion;
	}
	else
	{
		if(InOutToken.Owner)
		{
			UE_LOGF(LogChaosCache, Warning, "Attempted to close a recording session with a token from an external cache.");
		}
		else
		{
			UE_LOGF(LogChaosCache, Warning, "Attempted to close a recording session with an invalid token");
		}
	}

	if (bCompressChannels)
	{
		CompressChannelsData(ChannelsCompressionErrorThreshold, ChannelsCompressionSampleRate);
	}

	if (bChaosCacheCompressTracksAfterRecording)
	{
		CompressTracks();
	}
}

void UChaosCache::CompressTracks()
{
	for (FPerParticleCacheData& Track : ParticleTracks)
	{
		Track.TransformData.Compress();
	}
}

FCacheUserToken UChaosCache::BeginPlayback() const
{
	CurrentPlaybackCount++;
	if(CurrentRecordCount.load() == 0)
	{
		// We can playback from this cache as it isn't open for record
		return FCacheUserToken(true, false, this);
	}
	else
	{
		CurrentPlaybackCount--;
	}

	return FCacheUserToken(false, false, this);
}

void UChaosCache::EndPlayback(FCacheUserToken& InOutToken) const
{
	if(InOutToken.IsOpen() && InOutToken.Owner == this)
	{
		// Invalidate the token
		InOutToken.bIsOpen = false;
		InOutToken.Owner = nullptr;
		CurrentPlaybackCount--;
	}
	else
	{
		if(InOutToken.Owner)
		{
			UE_LOGF(LogChaosCache, Warning, "Attempted to close a playback session with a token from an external cache.");
		}
		else
		{
			UE_LOGF(LogChaosCache, Warning, "Attempted to close a playback session with an invalid token");
		}
	}
}

void UChaosCache::AddFrame_Concurrent(FPendingFrameWrite&& InFrame)
{
	PendingWrites.Enqueue(MoveTemp(InFrame));
}

float UChaosCache::GetDuration() const
{
	return RecordedDuration;
}

FCacheEvaluationResult UChaosCache::Evaluate(const FCacheEvaluationContext& InContext, const TArray<FTransform>* MassToLocalTransforms) const
{
	QUICK_SCOPE_CYCLE_COUNTER(QSTAT_CacheEval);

	FCacheEvaluationResult Result;
	static const TMap<FName, float> EmptyMap;

	if(CurrentPlaybackCount.load() == 0)
	{
		// No valid playback session
		UE_LOGF(LogChaosCache, Warning, "Attempted to evaluate a cache that wasn't opened for playback");
		return Result;
	}

	Result.EvaluatedTime = InContext.TickRecord.GetTime();

	if(!InContext.bEvaluateTransform && !InContext.bEvaluateCurves && !InContext.bEvaluateEvents && !InContext.bEvaluateChannels && !InContext.bEvaluateNamedTransforms)
	{
		// no evaluation requested
		return Result;
	}

	const int32 NumProvidedIndices = InContext.EvaluationIndices.Num();

	if(NumProvidedIndices > 0 && NumProvidedIndices < ParticleTracks.Num())
	{
		if(InContext.bEvaluateTransform)
		{
			Result.Transform.Init(FTransform::Identity, NumProvidedIndices);
		}

		if(InContext.bEvaluateCurves)
		{
			Result.Curves.Init(EmptyMap,NumProvidedIndices);
		}

		if(InContext.bEvaluateChannels)
		{
			for(auto& ChannelData : ChannelsTracks)
			{
				TArray<float>& ResultData = Result.Channels.FindOrAdd(ChannelData.Key);
				ResultData.Init(0.0, NumProvidedIndices);

				for(int32 EvalIndex = 0; EvalIndex < NumProvidedIndices; ++EvalIndex)
				{
					const int32 CacheIndex = InContext.EvaluationIndices[EvalIndex];
					ResultData[CacheIndex] = ChannelData.Value.RichCurves[CacheIndex].Eval(InContext.TickRecord.GetTime(),0.0);
				}
			}
			for(const TPair<FName, FCompressedRichCurves>& CompressedChannelData : CompressedChannelsTracks)
			{
				TArray<float>& ResultData = Result.Channels.FindOrAdd(CompressedChannelData.Key);
				ResultData.Init(0.0, NumProvidedIndices);
				
				for(int32 EvalIndex = 0; EvalIndex < NumProvidedIndices; ++EvalIndex)
				{
					ResultData[EvalIndex] = CompressedChannelData.Value.CompressedRichCurves[EvalIndex].Eval(InContext.TickRecord.GetTime(),0.0);
				}
			}
			if (ChannelCurveToParticle.Num() > 0)
			{
				ensure(ParticleTracks.Num() == 0);
				Result.ParticleIndices = ChannelCurveToParticle;
			}
		}

		for(int32 EvalIndex = 0; EvalIndex < NumProvidedIndices; ++EvalIndex)
		{
			const int32 CacheIndex = InContext.EvaluationIndices[EvalIndex];
			if(ensure(ParticleTracks.IsValidIndex(CacheIndex)))
			{
				FTransform* EvalTransform = nullptr;
				TMap<FName, float>* EvalCurves = nullptr;

				if(InContext.bEvaluateTransform)
				{
					EvalTransform = &Result.Transform[EvalIndex];
				}

				if(InContext.bEvaluateCurves)
				{
					EvalCurves = &Result.Curves[EvalIndex];
				}

				int32 ParticleIndex = INDEX_NONE;
				if(TrackToParticle.IsValidIndex(CacheIndex))
				{
					ParticleIndex = TrackToParticle[CacheIndex];
				}
				Result.ParticleIndices.Add(ParticleIndex);

				const FTransform* MassToLocal = (MassToLocalTransforms && ParticleIndex != INDEX_NONE)? &((*MassToLocalTransforms)[ParticleIndex]): nullptr;
				EvaluateSingle(CacheIndex, InContext.TickRecord, MassToLocal, EvalTransform, EvalCurves);
			}
		}
	}
	else
	{
		const int32 NumParticles = ParticleTracks.Num();
		
		if(InContext.bEvaluateTransform)
		{
			Result.Transform.Reserve(NumParticles);
		}

		if(InContext.bEvaluateCurves)
		{
			Result.Curves.Reserve(NumParticles);
		}
		if(InContext.bEvaluateChannels)
		{
			const int32 NumCurves = NumParticles > 0 ? NumParticles : ChannelCurveToParticle.Num();
			for(auto& ChannelData : ChannelsTracks)
			{
				TArray<float>& ResultData = Result.Channels.FindOrAdd(ChannelData.Key);
				ResultData.Init(0.0, NumCurves);
				
				for(int32 EvalIndex = 0; EvalIndex < NumCurves; ++EvalIndex)
				{
					ResultData[EvalIndex] = ChannelData.Value.RichCurves[EvalIndex].Eval(InContext.TickRecord.GetTime(),0.0);
				}
			}
			for(const TPair<FName, FCompressedRichCurves>& CompressedChannelData : CompressedChannelsTracks)
			{
				TArray<float>& ResultData = Result.Channels.FindOrAdd(CompressedChannelData.Key);
				ResultData.Init(0.0, NumCurves);
				
				for(int32 EvalIndex = 0; EvalIndex < NumCurves; ++EvalIndex)
				{
					ResultData[EvalIndex] = CompressedChannelData.Value.CompressedRichCurves[EvalIndex].Eval(InContext.TickRecord.GetTime(),0.0);
				}
			}

			if (ChannelCurveToParticle.Num() > 0)
			{
				ensure(ParticleTracks.Num() == 0);
				Result.ParticleIndices = ChannelCurveToParticle;
			}
		}

		for(int32 Index = 0; Index < NumParticles; ++Index)
		{
			if(ParticleTracks[Index].TransformData.BeginOffset > InContext.TickRecord.GetTime())
			{
				// Track hasn't begun yet so skip evaluation
				continue;
			}

			if (ParticleTracks[Index].TransformData.bDeactivateOnEnd && ParticleTracks[Index].TransformData.GetEndTime() < InContext.TickRecord.GetTime())
			{
				// Particle has deactivated so skip evaluation
				continue;
			}

			FTransform* EvalTransform = nullptr;
			TMap<FName, float>* EvalCurves = nullptr;

			if(InContext.bEvaluateTransform)
			{
				Result.Transform.Add(FTransform::Identity);
				EvalTransform = &Result.Transform.Last();
			}

			if(InContext.bEvaluateCurves)
			{
				Result.Curves.Add(EmptyMap);
				EvalCurves = &Result.Curves.Last();
			}

			int32 ParticleIndex = INDEX_NONE;
			if(TrackToParticle.IsValidIndex(Index))
			{
				ParticleIndex = TrackToParticle[Index];
			}
			Result.ParticleIndices.Add(ParticleIndex);

			const FTransform* MassToLocal = (MassToLocalTransforms && ParticleIndex != INDEX_NONE)? &((*MassToLocalTransforms)[ParticleIndex]): nullptr;
			EvaluateSingle(Index, InContext.TickRecord, MassToLocal, EvalTransform, EvalCurves);
		}
	}

	if(InContext.bEvaluateEvents)
	{
		Result.Events.Reserve(EventTracks.Num());
		EvaluateEvents(InContext.TickRecord, Result.Events);
	}

	if (InContext.bEvaluateNamedTransforms)
	{
		Result.NamedTransforms.Reset();
		for (const TPair<FName, FNamedTransformTrack>& NamedTransform : NamedTransformTracks)
		{
			Result.NamedTransforms.Add(NamedTransform.Key, NamedTransform.Value.Evaluate(InContext.TickRecord.GetTime(), nullptr, InterpolationMode));
		}
	}

	// Update the tick record on completion for the next run
	InContext.TickRecord.LastTime = InContext.TickRecord.GetTime();
	InContext.TickRecord.CurrentDt = 0.0f;

	return Result;
}

void UChaosCache::BuildSpawnableFromComponent(const UPrimitiveComponent* InComponent, const FTransform& SpaceTransform)
{
	Spawnable.DuplicatedTemplate = StaticDuplicateObject(InComponent, this);
	// duplication of the component also copy the attach parent reference, but if we keep it this causes crashes because the garbage collection will have a dangling reference
	// preventing the PIE level from being released properly and causing all sort of problem and crashes after that 
	if (USceneComponent* SceneComponent = Cast<USceneComponent>(Spawnable.DuplicatedTemplate))
	{
		if (SceneComponent->GetAttachParent())
		{
			SceneComponent->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform);
		}
	}
	// make sure we do not have any initialization fields because those may also keep dangling references preventing garbage collection
	if (UGeometryCollectionComponent* GeometryCollectionComponent = Cast<UGeometryCollectionComponent>(Spawnable.DuplicatedTemplate))
	{
		GeometryCollectionComponent->InitializationFields.Empty();
	}
	Spawnable.InitialTransform = InComponent->GetComponentToWorld();
	Spawnable.ComponentTransform = InComponent->GetComponentToWorld() * SpaceTransform.Inverse();
}

const FCacheSpawnableTemplate& UChaosCache::GetSpawnableTemplate() const
{
	return Spawnable;
}

void UChaosCache::EvaluateSingle(int32 InIndex, FPlaybackTickRecord& InTickRecord, const FTransform* MassToLocal, FTransform* OutOptTransform, TMap<FName, float>* OutOptCurves) const
{
	// check to satisfy SA, external callers check validity in Evaluate
	checkSlow(ParticleTracks.IsValidIndex(InIndex));
	const FPerParticleCacheData& Data = ParticleTracks[InIndex];

	
	if(OutOptTransform)
	{
		EvaluateTransform(Data, InTickRecord.GetTime(), MassToLocal, *OutOptTransform);
		(*OutOptTransform) = (*OutOptTransform) *InTickRecord.SpaceTransform;
	}
	

	if(OutOptCurves)
	{
		EvaluateCurves(Data, InTickRecord.GetTime(), *OutOptCurves);
	}
}

void UChaosCache::EvaluateTransform(const FPerParticleCacheData& InData, float InTime, const FTransform* MassToLocal, FTransform& OutTransform) const
{
	OutTransform = InData.TransformData.Evaluate(InTime, MassToLocal, InterpolationMode);
}

void UChaosCache::EvaluateCurves(const FPerParticleCacheData& InData, float InTime, TMap<FName, float>& OutCurves) const
{
	for(const TPair<FName, FRichCurve>& Curve : InData.CurveData)
	{
		OutCurves.FindOrAdd(Curve.Key) = Curve.Value.Eval(InTime, 0.0f);
	}
}

void UChaosCache::EvaluateEvents(FPlaybackTickRecord& InTickRecord, TMap<FName, TArray<FCacheEventHandle>>& OutEvents) const
{
	OutEvents.Reset();

	for(const TTuple<FName, FCacheEventTrack>& Track : EventTracks)
	{
		const FCacheEventTrack& TrackRef = Track.Value;
		if(TrackRef.Num() == 0)
		{
			continue;
		}

		int32* BeginIndexPtr = InTickRecord.LastEventPerTrack.Find(Track.Key);
		const int32 BeginIndex = BeginIndexPtr ? *BeginIndexPtr : 0;

		TConstArrayView<float> TimeStampView(&TrackRef.TimeStamps[BeginIndex], TrackRef.TimeStamps.Num() - BeginIndex);

		int32 BeginEventIndex = Algo::UpperBound(TimeStampView, InTickRecord.LastTime) + BeginIndex;
		const int32 EndEventIndex = Algo::UpperBound(TimeStampView, InTickRecord.GetTime()) + BeginIndex;

		TArray<FCacheEventHandle> NewHandles;
		NewHandles.Reserve(EndEventIndex - BeginEventIndex);

		// Add anything we found
		while(BeginEventIndex != EndEventIndex)
		{
			NewHandles.Add(TrackRef.GetEventHandle(BeginEventIndex));
			++BeginEventIndex;
		}

		// If we added any handles then we must have a new index for the lastevent tracker in the tick record
		if(NewHandles.Num() > 0)
		{
			int32& OutBeginIndex = BeginIndexPtr ? *BeginIndexPtr : InTickRecord.LastEventPerTrack.Add(Track.Key);
			OutBeginIndex = NewHandles.Last().Index;

			// Push to the result container
			OutEvents.Add(TTuple<FName, TArray<FCacheEventHandle>>(Track.Key, MoveTemp(NewHandles)));
		}
	}
}

void UChaosCache::CompressChannelsData(float ErrorThreshold, float SampleRate)
{
	for(TPair<FName, FRichCurves>& ChannelData : ChannelsTracks)
	{
		FCompressedRichCurves &CompressedRichCurves = CompressedChannelsTracks.FindOrAdd(ChannelData.Key);
		const int32 NumCurves = ChannelData.Value.RichCurves.Num();
		CompressedRichCurves.CompressedRichCurves.SetNum(NumCurves);
		::ParallelFor(NumCurves, [&ChannelData, &CompressedRichCurves, ErrorThreshold, SampleRate](int32 CurveIndex)
		{
			ChannelData.Value.RichCurves[CurveIndex].CompressCurve(CompressedRichCurves.CompressedRichCurves[CurveIndex], ErrorThreshold, SampleRate);
		}, false);
	}
	ChannelsTracks.Reset();
}

FTransform FParticleTransformTrack::Evaluate(float InCacheTime, const FTransform* MassToLocal, EChaosCacheInterpolationMode InterpolationMode) const
{
	QUICK_SCOPE_CYCLE_COUNTER(QSTAT_EvalParticleTransformTrack);
	const int32 NumKeys = GetNumKeys();

	FTransform Result{FTransform::Identity};
	if(NumKeys > 0)
	{
		const float ClampedTime = FMath::Clamp(InCacheTime, KeyTimestamps[0], KeyTimestamps.Last());  
		
		const int32 UpperKeyIndex = GetUpperBoundEvaluationIndex(ClampedTime);
		const int32 LowerKeyIndex = FMath::Max(0, (UpperKeyIndex-1));;
		
		const FTransform LowerTransform = EvaluateAt(LowerKeyIndex);
		const FTransform UpperTransform = EvaluateAt(UpperKeyIndex);
		
		const FTransform TransformA = (MassToLocal)? (*MassToLocal * LowerTransform): LowerTransform;
		const FTransform TransformB = (MassToLocal)? (*MassToLocal * UpperTransform): UpperTransform;
		
		const float Interval = KeyTimestamps[UpperKeyIndex] - KeyTimestamps[LowerKeyIndex];
		const float Alpha = (Interval > 0)? ((ClampedTime - KeyTimestamps[LowerKeyIndex]) / Interval): 0.0f;
		ensure(Alpha >= 0.f && Alpha <= 1.f); 

		if (bChaosCacheUseInterpolation)
		{
			if (InterpolationMode == EChaosCacheInterpolationMode::QuatInterp)
			{
				Result.Blend(TransformA, TransformB, Alpha);
			}
			else if (InterpolationMode == EChaosCacheInterpolationMode::EulerInterp)
			{
				Result.SetTranslation(FMath::Lerp(TransformA.GetTranslation(), TransformB.GetTranslation(), Alpha));
				Result.SetScale3D(FMath::Lerp(TransformA.GetScale3D(), TransformB.GetScale3D(), Alpha));

				const FRotator RotatorA = TransformA.Rotator();
				const FRotator RotatorB = TransformB.Rotator();
				const FRotator DeltaRotator = RotatorB - RotatorA;
				Result.SetRotation(FQuat(RotatorA + Alpha * DeltaRotator));
			}
			else // (InterpolationMode == EChaosCacheInterpolationMode::DualQuatInterp)
			{
				FTransform AdjustedTransformB(TransformB);
				if ((TransformB.GetRotation() | TransformA.GetRotation()) < 0.0f)
				{
					AdjustedTransformB.SetRotation(TransformB.GetRotation() * -1.0f);
				}
				Result = (FDualQuat(TransformA) * (1 - Alpha) + FDualQuat(AdjustedTransformB) * Alpha).Normalized().AsFTransform(FMath::Lerp(TransformA.GetScale3D(), AdjustedTransformB.GetScale3D(), Alpha));
			}
		}
		else
		{
			Result = TransformA;
		}
	}

	return Result;
}

int32 FParticleTransformTrack::GetUpperBoundEvaluationIndex(float InCacheTime) const
{
	if (KeyTimestamps.Num() == 0)
	{
		return 0;
	}
	return FMath::Clamp(Algo::UpperBound(KeyTimestamps, InCacheTime), 0, KeyTimestamps.Num()-1);
}

FTransform FParticleTransformTrack::EvaluateAt(int32 Index) const
{
	if (KeyTimestamps.Num() == 0)
	{
		return FTransform::Identity;
	}
	return FTransform{FQuat(RawTransformTrack.RotKeys[Index]), FVector(RawTransformTrack.PosKeys[Index])};
}

const int32 FParticleTransformTrack::GetNumKeys() const
{
	return KeyTimestamps.Num();
}

const float FParticleTransformTrack::GetDuration() const
{
	if(GetNumKeys() > 1)
	{
		return KeyTimestamps.Last() - KeyTimestamps[0];
	}
	return 0.0f;
}

const float FParticleTransformTrack::GetBeginTime() const
{
	const int32 NumKeys = GetNumKeys();
	if(NumKeys > 0)
	{
		return KeyTimestamps[0];
	}

	return 0.0f;
}

const float FParticleTransformTrack::GetEndTime() const
{
	const int32 NumKeys = GetNumKeys();
	if(NumKeys > 0)
	{
		return KeyTimestamps.Last();
	}

	return 0.0f;
}

void FParticleTransformTrack::CopyTrackEntry(int32 FromIndex, int32 ToIndex)
{
	if (FromIndex != ToIndex)
	{
		KeyTimestamps[ToIndex] = KeyTimestamps[FromIndex];
		RawTransformTrack.PosKeys[ToIndex] = RawTransformTrack.PosKeys[FromIndex];
		RawTransformTrack.RotKeys[ToIndex] = RawTransformTrack.RotKeys[FromIndex];
		// not sure we use that part anymore ( maybe in older caches ?)
		RawTransformTrack.ScaleKeys[ToIndex] = RawTransformTrack.ScaleKeys[FromIndex];
	}
}

void FParticleTransformTrack::ResizeTrack(int32 NewSize)
{
	if (NewSize < KeyTimestamps.Num())
	{
		KeyTimestamps.SetNum(NewSize);
		RawTransformTrack.PosKeys.SetNum(NewSize);
		RawTransformTrack.RotKeys.SetNum(NewSize);
		// not sure we use that part anymore ( maybe in older caches ?)
		RawTransformTrack.ScaleKeys.SetNum(NewSize);
	}
}

void FParticleTransformTrack::Compress()
{
	// we only need to compress if there's more than 3 keys
	if (KeyTimestamps.Num() >= 3)
	{
		// simple compression algorithm to remove similar keys
		// we compare the resulting transform
		// the compression can be done in place because the number of resulting keys is always smaller than the original number of keys
		// when more than 3 consecutive transforms are similar they are compressed using two ( earlier and oldest timestamp )
		// [A1 A2 A3 A4 B1 C1 C2 C3 D1 E1 E2 E3] will result in [A1 A4 B1 C1 C3 D1 E1 E3 ] 

		int32 CompressedKeyIndex = 0; // index to write to

		for (int32 KeyIndex = 0; KeyIndex < KeyTimestamps.Num(); KeyIndex++)
		{
			FTransform CurrentTransform = EvaluateAt(KeyIndex);

			// write current 
			CopyTrackEntry(KeyIndex, CompressedKeyIndex);
			CompressedKeyIndex++;

			// find the next index where the transform is different
			int32 NextIndex = (KeyIndex + 1);
			for (; NextIndex < KeyTimestamps.Num(); NextIndex++)
			{
				const FTransform NextTransform = EvaluateAt(NextIndex);
				if (!NextTransform.Equals(CurrentTransform))
				{
					break;
				}
			}

			// if there's at least one identical transform before the new one 
			const int32 LastSimilarIndex = (NextIndex - 1);
			if (LastSimilarIndex > KeyIndex)
			{
				// write same transform with oldest timestamp
				if (CompressedKeyIndex < KeyTimestamps.Num())
				{
					CopyTrackEntry(LastSimilarIndex, CompressedKeyIndex);
					CompressedKeyIndex++;
				}
				KeyIndex = (NextIndex - 1); // account for the automatic increment of the loop
			}
		}

		// we are done we can now shrink the original arrays to the compressed size
		const int32 CompressedSize = CompressedKeyIndex;
		ResizeTrack(CompressedSize);
	}
}
