// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/ChaosCachingPlugin.h"
#include "Chaos/Adapters/CacheAdapter.h"
#include "Chaos/Adapters/GeometryCollectionComponentCacheAdapter.h"
#include "Chaos/Adapters/StaticMeshComponentCacheAdapter.h"
#include "Chaos/Sequencer/ChaosCacheObjectSpawner.h"

IMPLEMENT_MODULE(IChaosCachingPlugin, ChaosCaching)

DEFINE_LOG_CATEGORY(LogChaosCache)

void IChaosCachingPlugin::StartupModule()
{
	GeometryCollectionAdapter = MakeUnique<Chaos::FGeometryCollectionCacheAdapter>();
	StaticMeshAdapter = MakeUnique<Chaos::FStaticMeshCacheAdapter>();

	RegisterAdapter(GeometryCollectionAdapter.Get());
	RegisterAdapter(StaticMeshAdapter.Get());
}

void IChaosCachingPlugin::ShutdownModule() 
{
	UnregisterAdapter(StaticMeshAdapter.Get());
	UnregisterAdapter(GeometryCollectionAdapter.Get());

	StaticMeshAdapter = nullptr;
	GeometryCollectionAdapter = nullptr;
}
