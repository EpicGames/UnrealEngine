// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Sequencer/ChaosCacheObjectSpawner.h"
#include "Chaos/CacheManagerActor.h"
#include "UObject/Package.h"
#include "Engine/World.h"

FChaosCacheObjectSpawner::FChaosCacheObjectSpawner() : FLevelSequenceActorSpawner()
{}

TSharedRef<IMovieSceneObjectSpawner> FChaosCacheObjectSpawner::CreateObjectSpawner()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return MakeShareable(new FChaosCacheObjectSpawner);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

UClass* FChaosCacheObjectSpawner::GetSupportedTemplateType() const
{
	return AChaosCacheManager::StaticClass();
}

UObject* FChaosCacheObjectSpawner::SpawnObject(FMovieSceneSpawnable& Spawnable, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState)
{
	UObject* SpawnedObject = FLevelSequenceActorSpawner::SpawnObject(Spawnable, TemplateID, SharedPlaybackState);
	
	if (AChaosCacheManager* ChaosCache = Cast<AChaosCacheManager>(SpawnedObject))
	{
		for(FObservedComponent& ObservedComponent : ChaosCache->GetObservedComponents())
		{
			FString FullPath = ObservedComponent.SoftComponentRef.OtherActor.ToString();

#if WITH_EDITORONLY_DATA
			if (const UPackage* ObjectPackage = SpawnedObject->GetPackage())
			{
				// If this is being set from PIE we need to remove the pie prefix 
				if (ObjectPackage->GetPIEInstanceID() == INDEX_NONE)
				{
					int32 PIEInstanceID = INDEX_NONE;
					FullPath = UWorld::RemovePIEPrefix(FullPath, &PIEInstanceID);
				}
			}
#endif
			ObservedComponent.SoftComponentRef.OtherActor = FSoftObjectPath(FullPath);
		}
	}

	return SpawnedObject;
}

void FChaosCacheObjectSpawner::DestroySpawnedObject(UObject& Object)
{
	FLevelSequenceActorSpawner::DestroySpawnedObject(Object);
}
