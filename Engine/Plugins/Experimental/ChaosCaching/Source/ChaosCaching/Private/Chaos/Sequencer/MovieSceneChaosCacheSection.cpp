// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Sequencer/MovieSceneChaosCacheSection.h"
#include "Chaos/CacheCollection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneChaosCacheSection)

FMovieSceneChaosCacheParams::FMovieSceneChaosCacheParams() : FMovieSceneBaseCacheParams()
{
	CacheCollection = nullptr;
}

float FMovieSceneChaosCacheParams::GetSequenceLength() const
{
	if(CacheCollection)
	{
		return CacheCollection->GetMaxDuration();
	}
	return 0.f;
}

UMovieSceneChaosCacheSection::UMovieSceneChaosCacheSection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ParamsPtr = &Params;
#if WITH_EDITOR
	InitializePlayRate();
#endif
}




