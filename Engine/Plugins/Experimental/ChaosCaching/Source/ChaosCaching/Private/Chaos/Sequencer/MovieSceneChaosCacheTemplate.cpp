// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneChaosCacheTemplate.h"

#include "Chaos/CacheCollection.h"
#include "Chaos/CacheManagerActor.h"
#include "Evaluation/MovieSceneExecutionTokens.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneChaosCacheTemplate)

DECLARE_CYCLE_STAT(TEXT("Chaos Cache Evaluate"), MovieSceneEval_ChaosCache_Evaluate, STATGROUP_MovieSceneEval);
DECLARE_CYCLE_STAT(TEXT("Chaos Cache Token Execute"), MovieSceneEval_ChaosCache_TokenExecute, STATGROUP_MovieSceneEval);

/** Used to set Manual Tick back to previous when outside section */
struct FPreAnimatedChaosCacheTokenProducer : IMovieScenePreAnimatedTokenProducer
{
	virtual IMovieScenePreAnimatedTokenPtr CacheExistingState(UObject& Object) const
	{
		struct FToken : IMovieScenePreAnimatedToken
		{
			FToken(AChaosCacheManager* InChaosCache)
			{
				// Cache this object's current update flag and animation mode
				StartMode = InChaosCache->StartMode;
				CacheMode = InChaosCache->CacheMode;
			}

			virtual void RestoreState(UObject& ObjectToRestore, const UE::MovieScene::FRestoreStateParams& Params)
			{
				AChaosCacheManager* ChaosCache = CastChecked<AChaosCacheManager>(&ObjectToRestore);
				ChaosCache->StartMode = StartMode;
				ChaosCache->CacheMode = CacheMode;
			}
			EStartMode StartMode;
			ECacheMode CacheMode;
			
		};

		return FToken(CastChecked<AChaosCacheManager>(&Object));
	}
	static FMovieSceneAnimTypeID GetAnimTypeID()
	{
		return TMovieSceneAnimTypeID<FPreAnimatedChaosCacheTokenProducer>();
	}
};

/** A movie scene execution token that executes a ChaosCache */
struct FChaosCacheExecutionToken : IMovieSceneExecutionToken
{
	FChaosCacheExecutionToken(const FMovieSceneChaosCacheSectionTemplateParameters &InParams) :
		Params(InParams)
	{}

	static AChaosCacheManager* ChaosCacheFromObject(UObject* BoundObject)
	{
		if (AActor* Actor = Cast<AActor>(BoundObject))
		{
			if (AChaosCacheManager* ChaosCache = Cast<AChaosCacheManager>(Actor))
			{
				return ChaosCache;
			}
		}
		return nullptr;
	}

	/** Execute this token, operating on all objects referenced by 'Operand' */
	virtual void Execute(const FMovieSceneContext& Context, const FMovieSceneEvaluationOperand& Operand, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) override
	{
		MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_ChaosCache_TokenExecute)
		if (Operand.ObjectBindingID.IsValid())
		{
			for (TWeakObjectPtr<> WeakObj : Player.FindBoundObjects(Operand))
			{
				if (UObject* Obj = WeakObj.Get())
				{
					AChaosCacheManager*  ChaosCache = ChaosCacheFromObject(Obj);
					if (ChaosCache && ChaosCache->CacheCollection)
					{ 
						// Set the ChaosCache on the component only if it's set and valid in the Params
						if (Params.ChaosCacheParams.CacheCollection && Params.ChaosCacheParams.CacheCollection != ChaosCache->CacheCollection)
						{
							ChaosCache->CacheCollection = Params.ChaosCacheParams.CacheCollection;
						}
						else
						{
							// It could be unset if the Params was referencing a transient ChaosCache
							// In that case, use the ChaosCache that is set on the component
							Params.ChaosCacheParams.CacheCollection = ChaosCache->CacheCollection;
						}
						Player.SavePreAnimatedState(*ChaosCache, FPreAnimatedChaosCacheTokenProducer::GetAnimTypeID(), FPreAnimatedChaosCacheTokenProducer());
						ChaosCache->StartMode = EStartMode::Timed;
						ChaosCache->CacheMode = ECacheMode::None;
						
						// calculate the time at which to evaluate the animation
						ChaosCache->SetStartTime( Params.MapTimeToAnimation(Params.ChaosCacheParams,
							ChaosCache->CacheCollection->GetMaxDuration(), Context.GetTime(), Context.GetFrameRate()));
					}
				}
			}
		}
	}

	FMovieSceneChaosCacheSectionTemplateParameters Params;
};

FMovieSceneChaosCacheSectionTemplate::FMovieSceneChaosCacheSectionTemplate(const UMovieSceneChaosCacheSection& InSection)
	: Params(const_cast<FMovieSceneChaosCacheParams&> (InSection.Params), InSection.GetInclusiveStartFrame(), InSection.GetExclusiveEndFrame())
{
}

//We use a token here so we can set the manual tick state back to what it was previously when outside this section.
//This is similar to how Skeletal Animation evaluation also works.
void FMovieSceneChaosCacheSectionTemplate::Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const
{
	MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_ChaosCache_Evaluate)
	ExecutionTokens.Add(FChaosCacheExecutionToken(Params));
}
