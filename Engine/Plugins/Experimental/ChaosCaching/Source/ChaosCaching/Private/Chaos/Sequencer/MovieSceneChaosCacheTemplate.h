// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Evaluation/MovieSceneBaseCacheTemplate.h"
#include "Chaos/Sequencer/MovieSceneChaosCacheSection.h"
#include "Evaluation/MovieSceneEvalTemplate.h"
#include "MovieSceneChaosCacheTemplate.generated.h"

USTRUCT()
struct FMovieSceneChaosCacheSectionTemplateParameters : public FMovieSceneBaseCacheSectionTemplateParameters
{
	GENERATED_BODY()

	FMovieSceneChaosCacheSectionTemplateParameters() {}

	FMovieSceneChaosCacheSectionTemplateParameters(const FMovieSceneChaosCacheParams& InChaosCacheParams, FFrameNumber InSectionStartTime, FFrameNumber InSectionEndTime)
		: FMovieSceneBaseCacheSectionTemplateParameters(InSectionStartTime, InSectionEndTime), ChaosCacheParams(InChaosCacheParams)
 
	{}

	UPROPERTY()
	FMovieSceneChaosCacheParams ChaosCacheParams;
};

USTRUCT()
struct FMovieSceneChaosCacheSectionTemplate : public FMovieSceneEvalTemplate
{
	GENERATED_BODY()

	FMovieSceneChaosCacheSectionTemplate() {}
	FMovieSceneChaosCacheSectionTemplate(const UMovieSceneChaosCacheSection& Section);

	virtual UScriptStruct& GetScriptStructImpl() const override { return *StaticStruct(); }
	virtual void Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const override;

	UPROPERTY()
	FMovieSceneChaosCacheSectionTemplateParameters Params;
};
