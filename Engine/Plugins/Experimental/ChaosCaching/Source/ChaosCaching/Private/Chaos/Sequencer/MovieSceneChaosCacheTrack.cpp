// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Sequencer/MovieSceneChaosCacheTrack.h"
#include "Chaos/CacheManagerActor.h"
#include "Chaos/CacheCollection.h"
#include "MovieScene.h"
#include "MovieSceneChaosCacheTemplate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneChaosCacheTrack)

#define LOCTEXT_NAMESPACE "MovieSceneChaosCacheTrack"

/* UMovieSceneChaosCacheTrack constructor
 *****************************************************************************/

UMovieSceneChaosCacheTrack::UMovieSceneChaosCacheTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(96, 128, 24, 65);
#endif

	SupportedBlendTypes.Add(EMovieSceneBlendType::Absolute);

	EvalOptions.bCanEvaluateNearestSection = true;
	EvalOptions.bEvaluateInPreroll = true;
}

/* UMovieSceneChaosCacheTrack interface
 *****************************************************************************/

UMovieSceneSection* UMovieSceneChaosCacheTrack::AddNewAnimation(FFrameNumber KeyTime, AChaosCacheManager* ChaosCache)
{
	UMovieSceneChaosCacheSection* NewSection = Cast<UMovieSceneChaosCacheSection>(CreateNewSection());
	{
		if(ChaosCache && ChaosCache->CacheCollection)
		{
			const FFrameTime AnimationLength = ChaosCache->CacheCollection->GetMaxDuration() * GetTypedOuter<UMovieScene>()->GetTickResolution();
			const int32 IFrameNumber = AnimationLength.FrameNumber.Value + static_cast<int32>(AnimationLength.GetSubFrame() + 0.5f) + 1;
			NewSection->InitialPlacementOnRow(AnimationSections, KeyTime, IFrameNumber, INDEX_NONE);
			
			NewSection->Params.CacheCollection = ChaosCache->CacheCollection;
		}
	}

	AddSection(*NewSection);

	return NewSection;
}

TArray<UMovieSceneSection*> UMovieSceneChaosCacheTrack::GetAnimSectionsAtTime(FFrameNumber Time)
{
	TArray<UMovieSceneSection*> Sections;
	for (auto Section : AnimationSections)
	{
		if (Section->IsTimeWithinSection(Time))
		{
			Sections.Add(Section);
		}
	}

	return Sections;
}

FMovieSceneEvalTemplatePtr UMovieSceneChaosCacheTrack::CreateTemplateForSection(const UMovieSceneSection& InSection) const
{
	return FMovieSceneChaosCacheSectionTemplate(*CastChecked<UMovieSceneChaosCacheSection>(&InSection));
}

/* UMovieSceneTrack interface
 *****************************************************************************/

const TArray<UMovieSceneSection*>& UMovieSceneChaosCacheTrack::GetAllSections() const
{
	return AnimationSections;
}

bool UMovieSceneChaosCacheTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneChaosCacheSection::StaticClass();
}

UMovieSceneSection* UMovieSceneChaosCacheTrack::CreateNewSection()
{
	return NewObject<UMovieSceneChaosCacheSection>(this, NAME_None, RF_Transactional);
}

void UMovieSceneChaosCacheTrack::RemoveAllAnimationData()
{
	AnimationSections.Empty();
}

bool UMovieSceneChaosCacheTrack::HasSection(const UMovieSceneSection& Section) const
{
	return AnimationSections.Contains(&Section);
}

void UMovieSceneChaosCacheTrack::AddSection(UMovieSceneSection& Section)
{
	AnimationSections.Add(&Section);
}

void UMovieSceneChaosCacheTrack::RemoveSection(UMovieSceneSection& Section)
{
	AnimationSections.Remove(&Section);
}

void UMovieSceneChaosCacheTrack::RemoveSectionAt(int32 SectionIndex)
{
	AnimationSections.RemoveAt(SectionIndex);
}

bool UMovieSceneChaosCacheTrack::IsEmpty() const
{
	return AnimationSections.Num() == 0;
}

#if WITH_EDITORONLY_DATA

FText UMovieSceneChaosCacheTrack::GetDefaultDisplayName() const
{
	return LOCTEXT("TrackName", "Chaos Cache");
}

#endif

#undef LOCTEXT_NAMESPACE
