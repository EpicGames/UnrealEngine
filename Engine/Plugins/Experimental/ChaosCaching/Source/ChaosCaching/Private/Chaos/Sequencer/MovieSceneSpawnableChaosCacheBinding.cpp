// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Sequencer/MovieSceneSpawnableChaosCacheBinding.h"
#include "Chaos/CacheManagerActor.h"
#include "Chaos/CacheCollection.h"
#include "UObject/Package.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSpawnableChaosCacheBinding)

#define LOCTEXT_NAMESPACE "MovieScene"

UObject* UMovieSceneSpawnableChaosCacheBinding::SpawnObjectInternal(UWorld* WorldContext, FName SpawnName, const FGuid& BindingId, int32 BindingIndex, UMovieScene& MovieScene, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState)
{
	UObject* SpawnedObject = Super::SpawnObjectInternal(WorldContext, SpawnName, BindingId, BindingIndex, MovieScene, TemplateID, SharedPlaybackState);

	if (AChaosCacheManager* ChaosCache = Cast<AChaosCacheManager>(SpawnedObject))
	{
		for (FObservedComponent& ObservedComponent : ChaosCache->GetObservedComponents())
		{
			FString FullPath = ObservedComponent.SoftComponentRef.OtherActor.ToString();

#if WITH_EDITORONLY_DATA
			if (const UPackage* ObjectPackage = SpawnedObject->GetPackage())
			{
				// If this is being set from PIE we need to remove the pie prefix 
				if (ObjectPackage->GetPIEInstanceID() == INDEX_NONE)
				{
					int32 PIEInstanceID = INDEX_NONE;
					FullPath = UWorld::RemovePIEPrefix(FullPath, &PIEInstanceID);
				}
			}
#endif
			ObservedComponent.SoftComponentRef.OtherActor = FSoftObjectPath(FullPath);
		}
	}

	return SpawnedObject;
}

bool UMovieSceneSpawnableChaosCacheBinding::SupportsBindingCreationFromObject(const UObject* SourceObject) const
{
	if (!SourceObject)
	{
		return false;
	}
	if (SourceObject->IsA<AChaosCacheManager>() || SourceObject->IsA<UChaosCacheCollection>())
	{
		return true;
	}

	return false;
}

#if WITH_EDITOR

FText UMovieSceneSpawnableChaosCacheBinding::GetBindingTypePrettyName() const
{
	return LOCTEXT("MovieSceneSpawnableChaosCacheBinding", "Spawnable Chaos Cache");
}
#endif

#undef LOCTEXT_NAMESPACE
