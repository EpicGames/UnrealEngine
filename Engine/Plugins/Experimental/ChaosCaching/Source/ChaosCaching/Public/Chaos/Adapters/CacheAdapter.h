// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Declares.h"
#include "Features/IModularFeature.h"
#include "Math/Transform.h"

namespace Chaos { class FPhysicsSolverEvents; }
template <class TClass> class TSubclassOf;

class UClass;
class UChaosCache;
class UPrimitiveComponent;
struct FPlaybackTickRecord;
struct FObservedComponent;

DECLARE_LOG_CATEGORY_EXTERN(LogCacheAdapter, Log, All);

struct FPendingFrameWrite;
class AChaosCacheManager;

namespace Chaos
{
	class FComponentCacheAdapter;
	struct FAdapterUtil
	{
		static CHAOSCACHING_API FComponentCacheAdapter* GetBestAdapterForClass(TSubclassOf<UPrimitiveComponent> InComponentClass, bool bAllowDerived = true);
	};

	/**
	 * Base adapter type for interfacing with Chaos simulation caches. Any component wishing to use caches must have a
	 * compatible record adapter and a playback adapter to produce and consume cache data.
	 *
	 * This base interface should not be used to implement those adapters however but the derived classes
	 * FCacheRecordAdapter and FCachePlaybackAdapter (declared below) should instead be used.
	 */
	class FComponentCacheAdapter : public IModularFeature
	{
	public:
		enum class SupportType : uint8
		{
			None,
			Direct,
			Derived
		};

		/** Registration name for modular features module */
		static CHAOSCACHING_API const FName FeatureName;
		static CHAOSCACHING_API const uint8 EngineAdapterPriorityBegin;
		static CHAOSCACHING_API const uint8 UserAdapterPriorityBegin;

		FComponentCacheAdapter()          = default;
		virtual ~FComponentCacheAdapter() = default;

		/**
		 * Query whether InComponentClass can be driven by this adapter. If a class implementing this interface says it
		 * can support a component it *may* be chosen as the driving adapter for that component and will be expeted to
		 * be able to handle Pull/Push operations if it is chosen.
		 *
		 * If the adapter returns SupportType::Derived and another adapter returns SupportType::Direct then the adapter
		 * that can directly drive the class is chosen to drive the cache. If multiple adapters return
		 * SupportType::Direct then the first is chosen. If multiple adapters return SupportType::Derived then the
		 * adapter with the closest desired class will be chosen (closest in terms of inheritance hierarchy between the
		 * desired direct class and the provided class)
		 *
		 * If the adapter can only handle one specific type then only using SupportType::Direct and SupportType::None
		 * will make sure it only receives calls from the cache if exactly that component class is selected
		 *
		 * @param InComponent - the component class to test
		 *
		 * @return the type of support this adapter can give for the specified class
		 */
		virtual SupportType SupportsComponentClass(UClass* InComponentClass) const = 0;

		/**
		 * Query the class that this adapter wants to drive. This will be used to work out the most likely candidate for
		 * an adapter when multiple adapters return SupportType::Derived. For example in the following hierarchy:
		 *
		 *           UPrimitiveComponent
		 *                   |
		 *           UDerivedComponentA
		 *                   |
		 *           UDerivedComponentB
		 *
		 * An adapter directly supporing UDerivedComponentA that returns SupportType::Derived for a class of
		 * UDerivedComponentB will be picked over an adapter that directly supports UPrimitiveComponent and returns
		 * SupportType::Derived for a class of UDerivedComponentB
		 *
		 * Note: All classes returned from this MUST have UPrimitiveComponent in their hierarchy as that is the base
		 * physically-capable component
		 */
		virtual UClass* GetDesiredClass() const = 0;

		/**
		 * Gets the priority for an adapter.
		 * When two or more adapters give the same support level for a given component class this priority will be
		 * used to decide which adapter will be used. All base engine level adapters per type will use a priority
		 * between EngineAdapterPriorityBegin and UserAdapterPriorityBegin defined in this interface.
		 *
		 * Users implementing adapters intended to override all engine functionality should return priorities
		 * above UserAdapterPriorityBegin to ensure they will always be selected above engine adapters.
		 */
		virtual uint8 GetPriority() const = 0;

		/**
		 * Called to retrieve the rigid solver for a specific component if it exists. Required until a more generic method
		 * of solver binding for components is devised.
		 * #BGTODO Remove when multiple solver concept moved into primitive component
		 * @param InComponent Component to resolve the solver for
		 */
		virtual Chaos::FPhysicsSolver* GetComponentSolver(UPrimitiveComponent* InComponent) const = 0;

		/**
		 * Called to retrieve the base events solver for a specific component. If the component is
		 * requiring it the solver could be rebuilt.
		 * @param InComponent Component to resolve the solver for
		 */
		virtual Chaos::FPhysicsSolverEvents* BuildEventsSolver(UPrimitiveComponent* InComponent) const { return nullptr; }

		/**
		 * Called from the game thread to perform any global setup that the adapter may need to perform.
		 */
		virtual void Initialize() {}

		/**
		 * Called from the game thread to perform any final steps before deallocation.
		 */
		virtual void Finalize() {}

		/**
		* Called from the game thread to set rest state from an evaluated time in the cache.
		*/
		virtual void SetRestState(UPrimitiveComponent* InComponent, UChaosCache* InCache, const FTransform& InRootTransform, Chaos::FReal InTime) const = 0;

		/**
		* Called from the game thread to emit events
		*/
		virtual void EmitEvents(UPrimitiveComponent* InComponent, UChaosCache* InCache, FPlaybackTickRecord& TickRecord) const {};

		/**
		 * Called from the game thread to initialize a component and cache ready to load a cache at a given frame
		 */
		virtual void InitializeForLoad(UPrimitiveComponent* InComponent, FObservedComponent& InObserved) {}
		
		/**
		 * Called from the game thread to initialize a component and cache ready to record a cache
		 * @param InComponent Target component to initialize
		 * @param InCache Target cache to initialize
		 */
		virtual bool InitializeForRecord(UPrimitiveComponent* InComponent, UChaosCache* InCache) { unimplemented(); return false; }
		virtual bool CHAOSCACHING_API InitializeForRecord(UPrimitiveComponent* InComponent, FObservedComponent& InObserved);

		/**
		 * Called from the game thread to initialize a component and cache ready to playback a cache
		 * @param InComponent Target component to initialize
		 * @param InCache Target cache to initialize
		 * @param InTime Time from cache start to evaluate initial conditions
		 */
		virtual bool InitializeForPlayback(UPrimitiveComponent* InComponent, UChaosCache* InCache, float InTime) { unimplemented(); return false; }
		virtual bool CHAOSCACHING_API InitializeForPlayback(UPrimitiveComponent* InComponent, FObservedComponent& InObserved, float InTime);

		/**
		* Called from the game thread to finalize for a specific component
		*/
		virtual void FinalizeComponent(UPrimitiveComponent* InComponent, UChaosCache* InCache) {}

		/**
		 * Called by a cache observer when a component should be recorded to a cache through this adapter.
		 * The time provided is the absolute time from the beginning of the cache recording that the adapter is
		 * expected to record to. Almost always recording should be done from Record_PostSolve
		 */
		virtual void Record_PostSolve(UPrimitiveComponent* InComp, const FTransform& InRootTransform, FPendingFrameWrite& OutFrame, Chaos::FReal InTime) const = 0;

		/**
		 * Called by a cache observer actor when a cache needs to be applied to a component through this adapter.
		 * The time provided is the absolute time from the beginning of the cache playback that the adapter is
		 * expected to apply to the supplied component. Note this is called on the physics thread.
		 */
		virtual void Playback_PreSolve(UPrimitiveComponent*                               InComponent,
									   UChaosCache*                                       InCache,
									   Chaos::FReal                                       InTime,
									   FPlaybackTickRecord&                               TickRecord,
									   TArray<TPBDRigidParticleHandle<Chaos::FReal, 3>*>& OutUpdatedRigids) const = 0;

		/**
		 * Gets a unique identifier for the adapter
		 *
		 * This must return a stable GUID that uniquely identifies any derived classes. This GUID is embedded into
		 * any cache that an adapter writes to so it can be matched up on replay. Changing the GUID returned will
		 * permanently invalidate any caches that were recorded with this adapter and they will no longer be able
		 * to playback
		 */
		virtual FGuid GetGuid() const = 0;

		/**
		 * Determines whether a cache is able to safely playback on a component.
		 * After the GUID for an adapter is validated this check will be called from the game thread to ensure
		 * an adapter is able to safely play back for the provided component.
		 * @param InComponent Target component attempting playback.
		 * @param InCache Requested cache to playback.
		 */
		virtual bool ValidForPlayback(UPrimitiveComponent* InComponent, UChaosCache* InCache) const = 0;

		/**
		 * if the cache adapter supports playback when recording
		 */
		virtual bool SupportsPlayWhileRecording() const { return false;  };
		virtual bool SupportsRestartSimulation() const { return false; };
		void SetRestartSimulation(bool InbRestartSimulation) { bRestartSimulation = InbRestartSimulation; };
		void SetRestartTimeRange(float InRestartTimeStart, float InRestartTimeEnd) {
			RestartTimeStart = InRestartTimeStart; RestartTimeEnd = InRestartTimeEnd;
		};

		/**
		* Stall on any solver tasks that could broadcast events via the EventsSolver. Called from game thread.
		* This is necessary so that the cache manager can teardown and setup new event handling.
		*/
		virtual void WaitForSolverTasks(UPrimitiveComponent* InComponent) const {};

	protected:
		bool bRestartSimulation = false;
		float RestartTimeStart = TNumericLimits<float>::Max(); // if bRestartSimulation, use RestartTimeStart as an offset when saving, and clear cache data in [RestartTimeStart, RestartTimeEnd] 
		float RestartTimeEnd = -TNumericLimits<float>::Max();

	private:
	};

	void CHAOSCACHING_API RegisterAdapter(FComponentCacheAdapter* InAdapter); 
	void CHAOSCACHING_API UnregisterAdapter(FComponentCacheAdapter* InAdapter);

	/** Helper to handle automatic global registration for adapter types - with type checking for valid adapter types */
	template<typename AdapterType>
	struct TAutoRegisterCacheAdapter
	{
		static_assert(TPointerIsConvertibleFromTo<AdapterType, FComponentCacheAdapter>::Value, "AdapterType is not an adapter");

		TAutoRegisterCacheAdapter()
		{
			RegisterAdapter(&Adapter);
		}

		~TAutoRegisterCacheAdapter()
		{
			UnregisterAdapter(&Adapter);
		}

	private:
		AdapterType Adapter;
	};

}    // namespace Chaos
