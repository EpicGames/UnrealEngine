// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CacheAdapter.h"
#include "Chaos/CacheEvents.h"
#include "Chaos/Framework/PhysicsSolverBase.h"
#include "EventsData.h"
#include "GeometryCollectionComponentCacheAdapter.generated.h"

USTRUCT()
struct FEnableStateEvent : public FCacheEventBase
{
	GENERATED_BODY()

	static FName EventName;

	FEnableStateEvent()
		: Index(INDEX_NONE)
		, bEnable(false)
	{
	}

	FEnableStateEvent(int32 InIndex, bool bInEnable)
		: Index(InIndex)
		, bEnable(bInEnable)
	{
	}

	UPROPERTY()
	int32 Index;

	UPROPERTY()
	bool bEnable;
};

USTRUCT()
struct FBreakingEvent : public FCacheEventBase
{
	GENERATED_BODY()

	static FName EventName;

	FBreakingEvent()
		: Index(INDEX_NONE)
		, Location(0.0f, 0.0f, 0.0f)
		, Orientation(FQuat::Identity)
		, Velocity(0.0f, 0.0f, 0.0f)
		, AngularVelocity(0.0f, 0.0f, 0.0f)
		, Mass(1.0f)
		, BoundingBoxMin(0.0f, 0.0f, 0.0f)
		, BoundingBoxMax(0.0f, 0.0f, 0.0f)
	{ }

	FBreakingEvent(int32 InIndex, const Chaos::FBreakingData& InData, const FTransform& WorldToComponent)
		: Index(InIndex)
		, Location(WorldToComponent.TransformPosition(InData.Location))
		, Orientation(WorldToComponent.TransformRotation(InData.Orientation))
		, Velocity(WorldToComponent.TransformVector(InData.Velocity))
		, AngularVelocity(InData.AngularVelocity)
		, Mass(InData.Mass)
		, BoundingBoxMin(InData.BoundingBox.Min())
		, BoundingBoxMax(InData.BoundingBox.Max())
	{
		FBox TransformedBounds = FBox(BoundingBoxMin, BoundingBoxMax).TransformBy(WorldToComponent);
		BoundingBoxMin = TransformedBounds.Min;
		BoundingBoxMax = TransformedBounds.Max;
	}

	UPROPERTY()
	int32 Index;

	UPROPERTY()
	FVector Location;
	
	UPROPERTY()
	FQuat Orientation;

	UPROPERTY()
	FVector Velocity;

	UPROPERTY()
	FVector AngularVelocity;
	
	UPROPERTY()
	float Mass;

	UPROPERTY()
	FVector BoundingBoxMin;

	UPROPERTY()
	FVector BoundingBoxMax;
};

USTRUCT()
struct FCollisionEvent : public FCacheEventBase
{
	GENERATED_BODY()

	static FName EventName;

	FCollisionEvent()
		: Location(0.0f, 0.0f, 0.0f)
		, AccumulatedImpulse(0.0f, 0.0f, 0.0f)
		, Normal(0.0f, 0.0f, 1.0f)
		, Velocity1(0.0f, 0.0f, 0.0f)
		, Velocity2(0.0f, 0.0f, 0.0f)
		, DeltaVelocity1(0.0f, 0.0f, 0.0f)
		, DeltaVelocity2(0.0f, 0.0f, 0.0f)
		, AngularVelocity1(0.0f, 0.0f, 0.0f)
		, AngularVelocity2(0.0f, 0.0f, 0.0f)
		, Mass1(1.0f)
		, Mass2(1.0f)
		, PenetrationDepth(0.0f)
	{}

	FCollisionEvent(const Chaos::FCollidingData& InData, const FTransform& WorldToComponent)
		: Location(WorldToComponent.TransformPosition(InData.Location))
		, AccumulatedImpulse(WorldToComponent.TransformVector(InData.AccumulatedImpulse))
		, Normal(WorldToComponent.TransformVector(InData.Normal))
		, Velocity1(WorldToComponent.TransformVector(InData.Velocity1))
		, Velocity2(WorldToComponent.TransformVector(InData.Velocity2))
		, DeltaVelocity1(WorldToComponent.TransformVector(InData.DeltaVelocity1))
		, DeltaVelocity2(WorldToComponent.TransformVector(InData.DeltaVelocity2))
		, AngularVelocity1(InData.AngularVelocity1)
		, AngularVelocity2(InData.AngularVelocity2)
		, Mass1(InData.Mass1)
		, Mass2(InData.Mass2)
		, PenetrationDepth(InData.PenetrationDepth)
	{}
	
	UPROPERTY()
	FVector Location;
	
	UPROPERTY()
	FVector AccumulatedImpulse;
	
	UPROPERTY()
	FVector Normal;
	
	UPROPERTY()
	FVector Velocity1;
	
	UPROPERTY()
	FVector Velocity2;
	
	UPROPERTY()
	FVector DeltaVelocity1;
	
	UPROPERTY()
	FVector DeltaVelocity2;
	
	UPROPERTY()
	FVector AngularVelocity1;
	
	UPROPERTY()
	FVector AngularVelocity2;
	
	UPROPERTY()
	float Mass1;
	
	UPROPERTY()
	float Mass2;
	
	UPROPERTY()
	float PenetrationDepth;
};

USTRUCT()
struct FTrailingEvent : public FCacheEventBase
{
	GENERATED_BODY()

	static FName EventName;

	FTrailingEvent()
		: Index(INDEX_NONE)
		, Location(0.0f, 0.0f, 0.0f)
		, Orientation(FQuat::Identity)
		, Velocity(0.0f, 0.0f, 0.0f)
		, AngularVelocity(0.0f, 0.0f, 0.0f)
		, BoundingBoxMin(0.0f, 0.0f, 0.0f)
		, BoundingBoxMax(0.0f, 0.0f, 0.0f)
	{ }

	FTrailingEvent(int32 InIndex, const Chaos::FTrailingData& InData, const FTransform& WorldToComponent)
		: Index(InIndex)
		, Location(WorldToComponent.TransformPosition(InData.Location))
		, Orientation(WorldToComponent.TransformRotation(InData.Orientation))
		, Velocity(WorldToComponent.TransformVector(InData.Velocity))
		, AngularVelocity(InData.AngularVelocity)
		, BoundingBoxMin(InData.BoundingBox.Min())
		, BoundingBoxMax(InData.BoundingBox.Max())
	{
		FBox TransformedBounds = FBox(BoundingBoxMin, BoundingBoxMax).TransformBy(WorldToComponent);
		BoundingBoxMin = TransformedBounds.Min;
		BoundingBoxMax = TransformedBounds.Max;
	}

	UPROPERTY()
	int32 Index;

	UPROPERTY()
	FVector Location;

	UPROPERTY()
	FQuat Orientation;

	UPROPERTY()
	FVector Velocity;

	UPROPERTY()
	FVector AngularVelocity;

	UPROPERTY()
	FVector BoundingBoxMin;

	UPROPERTY()
	FVector BoundingBoxMax;
};


class UChaosCache;
class UPrimitiveComponent;
struct FCacheEvaluationResult;

namespace Chaos
{
	// forward declaration to avoid adding EventManager.h
	enum class EEventType : int32;

	struct FCachedEventData
	{
		FCachedEventData()
			: ProxyBreakingDataIndices(nullptr)
			, ProxyCollisionDataIndices(nullptr)
			, ProxyTrailingDataIndices(nullptr)
		{ }

		const TArray<int32>* ProxyBreakingDataIndices;
		const TArray<int32>* ProxyCollisionDataIndices;
		const TArray<int32>* ProxyTrailingDataIndices;
	};
	
	class FGeometryCollectionCacheAdapter : public FComponentCacheAdapter
	{
	public:

		virtual ~FGeometryCollectionCacheAdapter() = default;

		// FComponentCacheAdapter interface
		virtual SupportType            SupportsComponentClass(UClass* InComponentClass) const override;
		virtual UClass*                GetDesiredClass() const override;
		virtual uint8                  GetPriority() const override;
		virtual FGuid                  GetGuid() const override;
		virtual Chaos::FPhysicsSolver* GetComponentSolver(UPrimitiveComponent* InComponent) const override;
		virtual Chaos::FPhysicsSolverEvents* BuildEventsSolver(UPrimitiveComponent* InComponent) const override;
		virtual bool                   ValidForPlayback(UPrimitiveComponent* InComponent, UChaosCache* InCache) const override;
		virtual void				   SetRestState(UPrimitiveComponent* InComponent, UChaosCache* InCache, const FTransform& InRootTransform, Chaos::FReal InTime) const override;
		virtual void				   EmitEvents(UPrimitiveComponent* InComponent, UChaosCache* InCache, FPlaybackTickRecord& TickRecord) const override;
		virtual bool                   InitializeForRecord(UPrimitiveComponent* InComponent, UChaosCache* InCache) override;
		virtual bool                   InitializeForPlayback(UPrimitiveComponent* InComponent, UChaosCache* InCache, float InTime) override;
		virtual void				   FinalizeComponent(UPrimitiveComponent* InComponent, UChaosCache* InCache) override;
		virtual void                   Record_PostSolve(UPrimitiveComponent* InComp, const FTransform& InRootTransform, FPendingFrameWrite& OutFrame, Chaos::FReal InTime) const override;
		virtual void                   Playback_PreSolve(UPrimitiveComponent*								InComponent,
												 UChaosCache*										InCache,
												 Chaos::FReal                                       InTime,
												 FPlaybackTickRecord&								TickRecord,
												 TArray<TPBDRigidParticleHandle<Chaos::FReal, 3>*>& OutUpdatedRigids) const override;

		// End FComponentCacheAdapter interface

	protected:

		void RegisterEventHandler(FGeometryCollectionPhysicsProxy* Proxy);
		void UnregisterEventHandler(FGeometryCollectionPhysicsProxy* Proxy);
		void HandleBreakingEvents(const Chaos::FBreakingEventData& Event);
		void HandleCollisionEvents(const Chaos::FCollisionEventData& Event);
		void HandleTrailingEvents(const Chaos::FTrailingEventData& Event);
		
		void EmitCollisionEvents_GameThread(UPrimitiveComponent& InComponent, const FCacheEvaluationResult& Results) const;
		void EmitBreakingEvents_GameThread(UPrimitiveComponent& InComponent, const FCacheEvaluationResult& Results) const;


	private:
		TArray<int32> GatherAllBreaksUpToTime(UChaosCache* InCache, float InTime) const;

		/** 
		* List of the solvers we are currently listening for events for recording
		* we need to track them because each of them owns a seperate unique event manager 
		*/
		using FSolverEventPair = TPair<Chaos::FPhysicsSolver*, Chaos::EEventType>;
		TSet<FSolverEventPair> SolversListenedToForEvents;

		TMap<IPhysicsProxyBase*, FCachedEventData> CachedData;
		const FBreakingDataArray* BreakingDataArray;
		const FCollisionDataArray* CollisionDataArray;
		const FTrailingDataArray* TrailingDataArray;


	};
}    // namespace Chaos
