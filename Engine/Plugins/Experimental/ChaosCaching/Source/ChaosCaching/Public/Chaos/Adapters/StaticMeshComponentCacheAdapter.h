// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CacheAdapter.h"
#include "Chaos/Framework/PhysicsSolverBase.h"

namespace Chaos
{
	class FStaticMeshCacheAdapter : public FComponentCacheAdapter
	{
	public:
		virtual ~FStaticMeshCacheAdapter() = default;

		virtual SupportType            SupportsComponentClass(UClass* InComponentClass) const override;
		virtual UClass*                GetDesiredClass() const override;
		virtual uint8                  GetPriority() const override;
		virtual FGuid                  GetGuid() const override;
		virtual bool                   ValidForPlayback(UPrimitiveComponent* InComponent, UChaosCache* InCache) const override;
		virtual Chaos::FPhysicsSolver* GetComponentSolver(UPrimitiveComponent* InComponent) const override;
		virtual Chaos::FPhysicsSolverEvents* BuildEventsSolver(UPrimitiveComponent* InComponent) const override;
		virtual void				   SetRestState(UPrimitiveComponent* InComponent, UChaosCache* InCache, const FTransform& InRootTransform, Chaos::FReal InTime) const override;
		virtual bool                   InitializeForRecord(UPrimitiveComponent* InComponent, UChaosCache* InCache) override;
		virtual bool                   InitializeForPlayback(UPrimitiveComponent* InComponent, UChaosCache* InCache, float InTime) override;
		virtual void                   Record_PostSolve(UPrimitiveComponent* InComp, const FTransform& InRootTransform, FPendingFrameWrite& OutFrame, Chaos::FReal InTime) const override;
		virtual void                   Playback_PreSolve(UPrimitiveComponent* InComponent,
												 UChaosCache*										InCache,
												 Chaos::FReal                                       InTime,
												 FPlaybackTickRecord&								TickRecord,
												 TArray<TPBDRigidParticleHandle<Chaos::FReal, 3>*>& OutUpdatedRigids) const override;
	private:
	};
}    // namespace Chaos
