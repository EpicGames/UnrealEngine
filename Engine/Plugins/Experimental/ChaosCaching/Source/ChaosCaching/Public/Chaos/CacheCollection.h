// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/ChaosCacheInterpolationMode.h"

#include "CacheCollection.generated.h"

class UChaosCache;

UCLASS(Experimental, BlueprintType, MinimalAPI)
class UChaosCacheCollection : public UObject
{
	GENERATED_BODY()
public:

	//~ Begin UObject Interface
	CHAOSCACHING_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	CHAOSCACHING_API virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;
#if WITH_EDITOR
	CHAOSCACHING_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent);
#endif
	//~ End UObject Interface
	
	CHAOSCACHING_API UChaosCache* FindCache(const FName& CacheName) const;
	CHAOSCACHING_API UChaosCache* FindOrAddCache(const FName& CacheName);

	CHAOSCACHING_API void FlushAllCacheWrites();

	const TArray<UChaosCache*>& GetCaches() const {return Caches;}

	/** Return the max duration of all the caches stored in the collection */
	CHAOSCACHING_API float GetMaxDuration() const;

	/** Return the max number of frames of all the caches stored in the collection */
	CHAOSCACHING_API uint32 GetMaxNumFrames() const;

	/** Set the interpolation mode on this cache collection */
	CHAOSCACHING_API void SetInterpolationMode(EChaosCacheInterpolationMode Mode);

	EChaosCacheInterpolationMode GetInterpolationMode() const { return InterpolationMode; }

	UPROPERTY(EditAnywhere, Instanced, Category="Caching", meta=(EditFixedOrder))
	TArray<TObjectPtr<UChaosCache>> Caches;

private:
	UPROPERTY(EditAnywhere, Category = "Caching")
	EChaosCacheInterpolationMode InterpolationMode;
};
