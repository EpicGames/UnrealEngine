// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/ChaosCachingPlugin.h"

#include "CacheEvents.generated.h"

/**
 * Base type for all events, ALL events must derive from this so we have a fallback for serialization
 * when we can't find the actual event struct.
 */
USTRUCT()
struct FCacheEventBase
{
	GENERATED_BODY();

	FCacheEventBase() {}

	virtual ~FCacheEventBase() = default;
};

USTRUCT()
struct FCacheEventTrack
{
	GENERATED_BODY()

	struct FHandle
	{
		FHandle()
			: Track(nullptr)
			, Version(0)
			, Index(INDEX_NONE)
		{
		}

		template<typename T>
		UE_DEPRECATED(5.6, "Non-const access is deprecated. Use GetConst instead")
		T* Get() const
		{
			return IsAlive() ? const_cast<FCacheEventTrack*>(Track)->GetEvent<T>(Index) : nullptr;
		}

		template<typename T>
		const T* GetConst() const
		{
			return IsAlive() ? Track->GetEvent<T>(Index) : nullptr;
		}

	private:
		friend class UChaosCache;
		friend struct FCacheEventTrack;

		bool IsAlive() const;

		const FCacheEventTrack* Track;
		int32             Version;
		int32             Index;
	};

	FCacheEventTrack();
	FCacheEventTrack(FName InName, UScriptStruct* InStruct);

	~FCacheEventTrack();

	UPROPERTY()
	FName Name;

	/** Type of the event stored in this track. Must inherit FCacheEventBase */
	UPROPERTY()
	TObjectPtr<UScriptStruct> Struct;

	UPROPERTY()
	TArray<float> TimeStamps;

	TArray<uint8*> EventData;

	/** Pushes an event to the track, this will perform a copy of the event and store it inside the track */
	template<typename T>
	void PushEvent(float TimeStamp, const T& InEvent)
	{
		if(!Struct)
		{
			UE_LOGF(LogChaosCache, Error, "Attempted to add an event to a track that has no struct");
			return;
		}

		// Have to push the right kind of data
		check(T::StaticStruct() == Struct);
		PushEventInternal(TimeStamp, &InEvent);
	}

	template<typename T>
	T* GetEvent(int32 Index)
	{
		if(EventData.IsValidIndex(Index) && Struct)
		{
			check(T::StaticStruct() == Struct);
			return reinterpret_cast<T*>(EventData[Index]);
		}

		return nullptr;
	}

	template<typename T>
	const T* GetEvent(int32 Index) const
	{
		if (EventData.IsValidIndex(Index) && Struct)
		{
			check(T::StaticStruct() == Struct);
			return reinterpret_cast<const T*>(EventData[Index]);
		}

		return nullptr;
	}

	/** Get a handle to an event that can easily resolve the inner event type without knowing the track */
	FHandle GetEventHandle(int32 Index) const;

	template<typename T>
	TArray<T*> GetEvents(float T0, float T1, int32* OptOutBeginIndex = nullptr)
	{
		check(T::StaticStruct() == Struct);

		TArray<T*> OutEvents;
		int32      BeginIndex = Algo::LowerBound(TimeStamps, T0);
		int32      EndIndex   = Algo::LowerBound(TimeStamps, T1);

		if(TimeStamps.IsValidIndex(BeginIndex))
		{
			if(OptOutBeginIndex)
			{
				(*OptOutBeginIndex) = BeginIndex;
			}

			EndIndex                 = TimeStamps.IsValidIndex(EndIndex) ? EndIndex : TimeStamps.Num();
			const int32 NumOutEvents = EndIndex - BeginIndex;
			OutEvents.Reserve(NumOutEvents);

			for(int32 EventIndex = BeginIndex; EventIndex < EndIndex; ++EventIndex)
			{
				OutEvents.Add(reinterpret_cast<T*>(EventData[EventIndex]));
			}
		}

		return OutEvents;
	}

	/** Because the memory management for this track is manual, Destroy all will destroy the stored structs correctly*/
	void DestroyAll();

	/** Custom serialize handles generic event serialization */
	bool Serialize(FArchive& Ar);

	/** Merge the events from another track, leaving the other empty. The other track must be of the same event type */
	void Merge(FCacheEventTrack&& Other);

	/** The transient version changes whenever the size of the event array changes and invalidates all old event handles */
	int32 GetTransientVersion() const;

	int32 Num() const
	{
		return EventData.Num();
	}

private:
	/**
	 * Version/Modified counter to invalidate evaluation handles if something holds on to them over time
	 * Because handles are used during evaluation we don't need to store this per event
	 */
	int32 TransientVersion;
	void  PushEventInternal(float TimeStep, const void* Event);
	void  LoadEventsFromArchive(FArchive& Ar);
	void  SaveEventsToArchive(FArchive& Ar);
};

using FCacheEventHandle = FCacheEventTrack::FHandle;

template<>
struct TStructOpsTypeTraits<FCacheEventTrack> : public TStructOpsTypeTraitsBase2<FCacheEventTrack>
{
	enum
	{
		WithSerializer = true,
	};
};
