// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/PBDRigidsEvolutionFwd.h"
#include "ChaosCache.h"
#include "GameFramework/Actor.h"

#include "CacheManagerActor.generated.h"

namespace Chaos { class FPhysicsSolverEvents; }

class UChaosCacheCollection;
class UPrimitiveComponent;
class UBillboardComponent;

namespace Chaos
{
class FComponentCacheAdapter;
}

UENUM()
enum class ECacheMode : uint8
{
	None		UMETA(DisplayName = "Static Pose"),
	Play,		
	Record
};

UENUM()
enum class EStartMode : uint8
{
	Timed,
	Triggered,
};

namespace Chaos
{
	using FPhysicsSolver = FPBDRigidsSolver;
}

USTRUCT(BlueprintType)
struct FObservedComponent
{
	GENERATED_BODY()

	FObservedComponent()
		: CacheName(NAME_None)
		, bIsSimulating(true)
		, bPlaybackEnabled(true)
		, bHasNotifyBreaks(false)
		, Cache(nullptr)
		, BestFitAdapter(nullptr)
	{
	}

	FObservedComponent(const FObservedComponent& OtherComponent)
	{
		CacheName = OtherComponent.CacheName;
		bIsSimulating = OtherComponent.bIsSimulating;
		bHasNotifyBreaks = OtherComponent.bHasNotifyBreaks;
		bPlaybackEnabled = OtherComponent.bPlaybackEnabled;
		USDCacheDirectory = OtherComponent.USDCacheDirectory;
		bTriggered = OtherComponent.bTriggered;
		AbsoluteTime = OtherComponent.AbsoluteTime;
		TimeSinceTrigger = OtherComponent.TimeSinceTrigger;
		Cache = OtherComponent.Cache;
		TickRecord = OtherComponent.TickRecord;
		BestFitAdapter = OtherComponent.BestFitAdapter;
		SoftComponentRef = OtherComponent.SoftComponentRef;
	}

	FObservedComponent& operator=(const FObservedComponent& OtherComponent)
	{
		CacheName = OtherComponent.CacheName;
		bIsSimulating = OtherComponent.bIsSimulating;
		bHasNotifyBreaks = OtherComponent.bHasNotifyBreaks;
		bPlaybackEnabled = OtherComponent.bPlaybackEnabled;
		USDCacheDirectory = OtherComponent.USDCacheDirectory;
		bTriggered = OtherComponent.bTriggered;
		AbsoluteTime = OtherComponent.AbsoluteTime;
		TimeSinceTrigger = OtherComponent.TimeSinceTrigger;
		Cache = OtherComponent.Cache;
		TickRecord = OtherComponent.TickRecord;
		BestFitAdapter = OtherComponent.BestFitAdapter;
		SoftComponentRef = OtherComponent.SoftComponentRef;
		
		return *this;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	~FObservedComponent()
	{
		OnDestroying.ExecuteIfBound(*this);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Unique name for the cache, used as a key into the cache collection */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Caching")
	FName CacheName;

	/** Deprecated hard object reference. Not working with sequencer and take recorder since
	 all the pointers from any other packages are cleared out. Use TSoftObjectPtr instead. */
	UE_DEPRECATED(5.1, "This property is going to be deleted. Use the SoftComponentRef instead")
	UPROPERTY()
	FComponentReference ComponentRef;

	/** The component observed by this object for either playback or recording */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Caching", meta = (UseComponentPicker, AllowAnyActor))
	FSoftComponentReference SoftComponentRef;

	/** Capture of the initial state of the component before cache manager takes control. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Caching")
	bool bIsSimulating;

	/** Whether this component is enabled for playback, this allow a cache to hold many component but only replay some of them. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Caching")
	bool bPlaybackEnabled;

	/** USD cache directory, if supported for this simulated structure type. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Caching", meta=(ContentDir))
	FDirectoryPath USDCacheDirectory;

	/** 
	* Capture the state of bNotifyBreaks of the component before cache manager takes control. 
	* this is because when recording the cache needs the component to have bNotifyBreaks set on the component 
	* to be able to properly record when clusters are breaking into smaller pieces
	*/
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Caching")
	bool bHasNotifyBreaks;

	/** Post serialize function to transfer datas from the deprecated TObjectPtr -> TSoftObjectPtr */
	CHAOSCACHING_API void PostSerialize(const FArchive& Ar);

	/** Prepare runtime tick data for a new run */
	CHAOSCACHING_API void ResetRuntimeData(const EStartMode ManagerStartMode);

	/** Check if the Observed component is enabled for a specific cache mode */
	CHAOSCACHING_API bool IsEnabled(ECacheMode CacheMode) const;
	
	/** Gets the chaos cache from the observed component */
	UChaosCache* GetChaosCache() const {return Cache;}

	/** Gets the component from the internal component ref */
	CHAOSCACHING_API UPrimitiveComponent* GetComponent(AActor* OwningActor);
	CHAOSCACHING_API UPrimitiveComponent* GetComponent(AActor* OwningActor) const;
	UE_DEPRECATED(5.3, "Use GetComponent(OwningActor) instead.")
	UPrimitiveComponent* GetComponent() { return GetComponent(nullptr); }
	UE_DEPRECATED(5.3, "Use GetComponent(OwningActor) instead.")
	UPrimitiveComponent* GetComponent() const { return GetComponent(nullptr); }

private:
	friend class AChaosCacheManager;
	friend class Chaos::FComponentCacheAdapter;

	bool         bTriggered;          // Whether the observed component is active
	Chaos::FReal AbsoluteTime;        // Time since BeginPlay
	Chaos::FReal TimeSinceTrigger;    // Time since our trigger happened
	UChaosCache* Cache;               // Cache to play - picked up in BeginPlay on the manager.
	FPlaybackTickRecord TickRecord;   // Tick record to track where we are in playback

	Chaos::FComponentCacheAdapter* BestFitAdapter; // Cache a pointer to the best adapter, required for cache random access handling.

	DECLARE_DELEGATE_OneParam(FOnDestroying, const FObservedComponent&);
	FOnDestroying OnDestroying;
};

struct FPerSolverData
{
	/* Handles to solver events to push/pull cache data */
	FDelegateHandle PreSolveHandle;
	FDelegateHandle PreBufferHandle;
	FDelegateHandle PostSolveHandle;
	FDelegateHandle TeardownHandle;

	/** List of the tick records for each playback index, tracks where the last tick was */
	TArray<FPlaybackTickRecord> PlaybackTickRecords;
	/** List of indices for components tagged for playback - avoids iterating non playback components */
	TArray<int32> PlaybackIndices;
	/** List of indices for components tagged for record - avoids iterating non record components */
	TArray<int32> RecordIndices;
	/** List of particles in the solver that are pending a kinematic update to be pushed back to their owner */
	TArray<Chaos::TPBDRigidParticleHandle<Chaos::FReal, 3>*> PendingKinematicUpdates;
};

UCLASS(Experimental, MinimalAPI)
class AChaosCacheManager : public AActor
{
	GENERATED_BODY()

public:
	CHAOSCACHING_API AChaosCacheManager(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/**
	 * The Cache Collection asset to use for this observer. This can be used for playback and record simultaneously
	 * across multiple components depending on the settings for that component.
	 */
	UPROPERTY(EditInstanceOnly, Category = "Caching")
	TObjectPtr<UChaosCacheCollection> CacheCollection;

	/**
	* When on, each time the cache starts recording a new asset will be created as an increment of the previous one
	*/
	UPROPERTY(EditAnywhere, Category = "Caching", meta = (EditCondition = "CacheCollection != nullptr"))
	bool bCreateNewCacheCollectionAssetOnRecord = false;

	/** How to use the cache - playback or record */
	UPROPERTY(EditAnywhere, Category = "Caching")
	ECacheMode CacheMode;

	/**
	* How to trigger the cache record or playback, timed will start counting at BeginPlay, Triggered will begin
	* counting from when the owning cache manager is requested to trigger the cache action
	* @see AChaosCacheManager::TriggerObservedComponent
	*/
	UPROPERTY(EditAnywhere, Category = "Caching")
	EStartMode StartMode;

	/**
	* Defines the (random access) time that represents the rest pose of the components managed by this cache.
	* When in Play mode, the components are set to the state provided by the caches at this evaluated time.
	*/
	UPROPERTY(EditAnywhere, Interp, BlueprintReadWrite, Category = "Caching", meta=(SequencerTrackClass="/Script/MovieSceneTracks.MovieSceneFloatTrack"))
	float StartTime;

	/**
	* if true, cache will start automatically on begin play(playing or recording)
	* if false, the function Start must be used to tsrat the cache again ( and can be stopped again using Stop )
	*/
	UPROPERTY(EditAnywhere, Category = "Caching")
	bool bStartOnBeginPlay;

	/**
	* When in static pose, replay the breaking and collision events through the Chaos Event Relay
	*/
	UPROPERTY(EditAnywhere, Category = "Caching", meta = (EditCondition = "CacheMode == ECacheMode::None"))
	bool bReplayEventsInStaticPoseMode = true;

	/** AActor interface */
	CHAOSCACHING_API void TickActor(float DeltaTime, enum ELevelTick TickType, FActorTickFunction& ThisTickFunction) override;
	/** end AActor interface */

	/** UObject interface */
	CHAOSCACHING_API virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	friend class IChaosCachingEditorPlugin;
#endif

#if WITH_EDITOR
	bool ContainsProperty(const UStruct* Struct, const void* InProperty) const;
	CHAOSCACHING_API virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif

#if WITH_EDITOR
	CHAOSCACHING_API virtual void PreEditChange(class FProperty* PropertyAboutToChange) override;
	CHAOSCACHING_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	/** end UObject interface */

	CHAOSCACHING_API void SetCacheMode(ECacheMode NewCacheMode);

	UFUNCTION(BlueprintCallable, Category = "Caching")
	CHAOSCACHING_API bool IsRecording() const;

	/** 
	* Expose StartTime property to Sequencer. GetStartTime will be called on keys. 
	* Note this will be ignored for evaluating in Play mode ( for that case, make sure to use Start(time) instead as this will properly stop the physics thread evalution and restert it after setting the time ) 
	*/
	UFUNCTION(CallInEditor)
	CHAOSCACHING_API void SetStartTime(float InStartTime);

	/**
	* Used to set the start time of the cache from a blueprint 
	* Note this will be ignored for evaluating in Play mode ( for that case, make sure to use Start(time) instead as this will properly stop the physics thread evalution and restert it after setting the time )
	*/
	UFUNCTION(BlueprintCallable, Category = "Caching")
	CHAOSCACHING_API void SetCurrentTime(float CurrentTime);

	/* 
	* Start the cache evaluation at a specific time 
	* If the cache if already eveluating, stop it, set the time, then re-start 
	* only works when the cache is in Static Pose or Play mode
	*/
	UFUNCTION(BlueprintCallable, Category = "Caching")
	CHAOSCACHING_API void Start(float InStartTime = 0);

	/* 
	* stop the cache from eveluating 
	*/
	UFUNCTION(BlueprintCallable, Category = "Caching")
	CHAOSCACHING_API void Stop();

	/** 
	 * Resets all components back to the world space transform they had when the cache for them was originally recorded
	 * if one is available
	 */
	UFUNCTION(BlueprintCallable, Category = "Caching")
	CHAOSCACHING_API void ResetAllComponentTransforms();

	/**
	 * Resets the component at the specified index in the observed list back to the world space transform it had when the 
	 * cache for it was originally recorded if one is available
	 * @param InIndex Index of the component to reset
	 */
	UFUNCTION(BlueprintCallable, Category = "Caching")
	CHAOSCACHING_API void ResetSingleTransform(int32 InIndex);

#if WITH_EDITOR
	/**
	 * Set the component at the specified index in the observed array to be the selected component in the outliner.
	 * This will also make that component's owner the selected actor in the outliner.
	 */
	CHAOSCACHING_API void SelectComponent(int32 InIndex);
#endif

	/** Returns true if this cache manager is allowed to record caches. */
	bool CanRecord() const { return bCanRecord; }

	/** Initialize the cache adapters before playing/recording the cache. */
	CHAOSCACHING_API void BeginEvaluate();
	
	/** Clean the cache adapters after playing/recording the cache.  */
	CHAOSCACHING_API void EndEvaluate();

	/** Accessor to the manager observed components (read only) */
	const TArray<FObservedComponent>& GetObservedComponents() const {return ObservedComponents;}

	/** Accessor to the manager observed components (read/write) */
	TArray<FObservedComponent>& GetObservedComponents() {return ObservedComponents;}

	/** Find or add a primitive component to a cache manager */
	UFUNCTION(BlueprintCallable, Category = "Caching", meta = (DisplayName = "UpdateOrAddObservedComponent"))
	CHAOSCACHING_API void FindOrAddObservedComponent(UPrimitiveComponent* InComponent, const FName& CacheName = TEXT(""), const bool bTransferSimulationFlag = false);

	/** Remove a primitive component from the cache manager */
	UFUNCTION(BlueprintCallable, Category = "Caching")
	CHAOSCACHING_API void RemoveObservedComponent(UPrimitiveComponent* InComponent);
	
	/** Clear all the observed components */
	UFUNCTION(BlueprintCallable, Category = "Caching")
	CHAOSCACHING_API void ClearObservedComponents(bool bDestroyComponents = true);
	
#if WITH_EDITOR
	CHAOSCACHING_API void SetObservedComponentProperties(const ECacheMode& NewCacheMode);

	CHAOSCACHING_API void SetEditorIconVisibility(bool bVisibility);
#endif
	void SetRestartSimulation(bool InbRestartSimulation) { bRestartSimulation = InbRestartSimulation; };
	void SetRestartTimeRange(float InRestartTimeStart, float InRestartTimeEnd) {
		RestartTimeStart = InRestartTimeStart; RestartTimeEnd = InRestartTimeEnd;
	};
	void SetIsSimulating(bool InbIsSimulating) { bIsSimulating = InbIsSimulating; };
	/** sets state for all observed components at the restart time. */
	CHAOSCACHING_API void ReadRestartData();

protected:

	/** AActor interface */
	CHAOSCACHING_API void BeginPlay() override;
	CHAOSCACHING_API void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	CHAOSCACHING_API void Destroyed() override;
	/** End AActor interface */
	
	/**	Handles physics thread pre-solve (push kinematic data for components under playback) */
    CHAOSCACHING_API void HandlePreSolve(Chaos::FReal InDt, Chaos::FPhysicsSolverEvents* InSolver);
   
    /** Handles physics thread pre-buffer (mark dirty kinematic particles) */
    CHAOSCACHING_API void HandlePreBuffer(Chaos::FReal InDt, Chaos::FPhysicsSolverEvents* InSolver);
   
    /** Handles physics thread post-solve (record data for components under record) */
    CHAOSCACHING_API void HandlePostSolve(Chaos::FReal InDt, Chaos::FPhysicsSolverEvents* InSolver);
   
    /** Handles solver teardown due to solver destruction / stream-out */
    CHAOSCACHING_API void HandleTeardown(Chaos::FPhysicsSolverEvents* InSolver);
	
	/** Evaluates and sets state for all observed components at the specified time. */
	CHAOSCACHING_API void OnStartFrameChanged(Chaos::FReal InT);

	/**
	* change the cache collection for this player 
	* if the cache is playing or recording this will have no effect
	*/
	UFUNCTION(BlueprintCallable, Category = "Caching")
	CHAOSCACHING_API void SetCacheCollection(UChaosCacheCollection* InCacheCollection);

public:
	/**
	 * Triggers a component to play or record.
	 * If the cache manager has an observed component entry for InComponent and it is a triggered entry
	 * this will begin the playback or record for that component, otherwise no action is taken.
	 * @param InComponent Component to trigger
	 */
	UFUNCTION(BlueprintCallable, Category = "Caching")
	CHAOSCACHING_API void TriggerComponent(UPrimitiveComponent* InComponent);

	/**
	 * Triggers a component to play or record.
	 * Searches the observed component list for an entry matching InCacheName and triggers the
	 * playback or recording of the linked observed component
	 * @param InCacheName Cache name to trigger
	 */
	UFUNCTION(BlueprintCallable, Category = "Caching")
	CHAOSCACHING_API void TriggerComponentByCache(FName InCacheName);

	/** Triggers the recording or playback of all observed components */
	UFUNCTION(BlueprintCallable, Category = "Caching")
	CHAOSCACHING_API void TriggerAll();

protected:
	/** Enable playback for a specific component using its cache name */
	UFUNCTION(BlueprintCallable, Category = "Caching")
	CHAOSCACHING_API void EnablePlaybackByCache(FName InCacheName, bool bEnable);

	/** Enable playback for a specific component using its index in the list of observed component */
	UFUNCTION(BlueprintCallable, Category = "Caching")
	CHAOSCACHING_API void EnablePlayback(int32 Index, bool bEnable);

	CHAOSCACHING_API FObservedComponent* FindObservedComponent(UPrimitiveComponent* InComponent);
	CHAOSCACHING_API FObservedComponent& AddNewObservedComponent(UPrimitiveComponent* InComponent);

	// Determines if the actor is allowed to record a cache.
	bool bCanRecord;

	// true if observed components are actively simulating, dynamically or kinematically.
	bool bIsSimulating;

private:
	
	friend class UActorFactoryCacheManager; // Allows the actor factory to set up the observed list. See UActorFactoryCacheManager::PostSpawnActor

	using FTickObservedFunction = TUniqueFunction<void(UChaosCache*, FObservedComponent&, Chaos::FComponentCacheAdapter*)>;

	/**
	 * Helper function to apply a callable to observed components if they've been triggered, all of the Dt/time
	 * bookkeeping is handled in one place
	 * @param InIndices Index list of the observed components to update
	 * @param InDt Delta for the tick
	 * @param InCallable Callable to fire if the observed component is active
	 */
	CHAOSCACHING_API void TickObservedComponents(const TArray<int32>& InIndices, Chaos::FReal InDt, FTickObservedFunction InCallable);

	/**
	* Stall on any in flight solver tasks that might call callbacks. Necessary before editing callbacks.
	*/
	CHAOSCACHING_API void WaitForObservedComponentSolverTasks();

	/**
	* this creates a new cache collection asset based on the existing one and set it as the current 
	* this also find the editor counterpart actor and update the cache collection accordingly
	*/
	void CreateAndSetNewCacheAsset();

	void OnObservedComponentDestroyed(const FObservedComponent& Component);
	void RegisterObservedComponentsNotifiers();
	void UnregisterObservedComponentsNotifiers();
	void EnsureObservedComponentsNotifierRegistration();

	/** List of observed objects and their caches */
	UPROPERTY(EditAnywhere, Category = "Caching")
	TArray<FObservedComponent> ObservedComponents;

	/** 1-1 list of adapters for the observed components, populated on BeginEvaluate */
	TArray<Chaos::FComponentCacheAdapter*> ActiveAdapters;

	/** List of particles returned by the adapter as requiring a kinematic update */
	TMap<Chaos::FPhysicsSolverEvents*, FPerSolverData> PerSolverData;

	/** Lists of currently open caches that need to be closed when complete */
	TArray<TTuple<FCacheUserToken, UChaosCache*>> OpenRecordCaches;
	TArray<TTuple<FCacheUserToken, UChaosCache*>> OpenPlaybackCaches;

	// since the object persists
	float StartTimeAtBeginPlay;

	// previous start time so we can keep track of the elasped time between SetStartTime calls
	float PreviousStartTime;

	bool bRestartSimulation = false; // if restart for a specified time range
	float RestartTimeStart = TNumericLimits<float>::Max(); // if bRestartSimulation, use RestartTimeStart as an offset when saving, and clear cache data in [RestartTimeStart, RestartTimeEnd] 
	float RestartTimeEnd = -TNumericLimits<float>::Max();

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	TObjectPtr<UBillboardComponent> EditorIconComponent;
#endif

};

UCLASS(Experimental, MinimalAPI)
class AChaosCachePlayer : public AChaosCacheManager
{
	GENERATED_BODY()

public:
	CHAOSCACHING_API AChaosCachePlayer(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

};
