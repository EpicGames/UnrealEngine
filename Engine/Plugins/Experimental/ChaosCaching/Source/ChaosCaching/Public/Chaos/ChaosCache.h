// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimTypes.h"
#include "CacheEvents.h"
#include "Containers/Queue.h"
#include "Curves/RichCurve.h"
#include "UObject/Interface.h"

#include "Chaos/ChaosCacheInterpolationMode.h"

#include "ChaosCache.generated.h"


USTRUCT()
struct FParticleTransformTrack
{
	GENERATED_BODY()

	/**
	 * List of all the transforms this cache cares about, recorded from the simulated transforms of the particles
	 * observed by the adapter that created the cache
	 */
	UPROPERTY()
	FRawAnimSequenceTrack RawTransformTrack;

	/** The offset from the beginning of the cache that holds this track that the track starts */
	UPROPERTY()
	float BeginOffset = 0.f;

	/** If this flag is set true, the particle represented by this track deactives on the final keyframe. */
	UPROPERTY()
	bool bDeactivateOnEnd = false;

	/**
	 * The above raw track is just the key data and doesn't know at which time those keys are placed, this is
	 * a list of the timestamps for each entry in TransformTrack
	 */
	UPROPERTY()
	TArray<float> KeyTimestamps;

	/**
	 * Evaluates the transform track at the specified time, returning the evaluated transform. When in between
	 * keys translations will be linearly interpolated and rotations spherically interpolated
	 * @param InCacheTime Absolute time from the beginning of the entire owning cache to evaluate.
	 * @param MassToLocal if not null, the will be premultiplied to transform before interpolation
	 */
	FTransform Evaluate(float InCacheTime, const FTransform* MassToLocal, EChaosCacheInterpolationMode InterpolationMode =  EChaosCacheInterpolationMode::QuatInterp) const;

	/**
	 * Find the index the key where timestamp is directly above InCacheTime
	 * returned value is garanteed to be within the range of keys if there's any
	 * in the case there's no key in the track, 0 is returned
	 * @param InCacheTime Absolute time from the beginning of the entire owning cache to evaluate.
	 * @return Valid key index
	 */
	int32 GetUpperBoundEvaluationIndex(float InCacheTime) const;

	/**
	 * Evaluate the acche at a sepcific index
	 * this assumes that index is within the range of valid keys
	 * * if the track is empty identity transform is returned
	 */
	FTransform EvaluateAt(int32 Index) const;

	const int32 GetNumKeys() const;
	const float GetDuration() const;
	const float GetBeginTime() const;
	const float GetEndTime() const;

	void Compress();

private:
	void CopyTrackEntry(int32 FromIndex, int32 ToIndex);
	void ResizeTrack(int32 NewSize);
};

USTRUCT()
struct FPerParticleCacheData
{
	GENERATED_BODY()

	UPROPERTY()
	FParticleTransformTrack TransformData;

	/**
	 * Named curve data. This can be particle or other continuous curve data pushed by the adapter that created the
	 * cache. Any particle property outside of the transforms will be placed in this container with a suitable name for
	 * the property. Blueprints and adapters can add whatever data they need to this container.
	 */
	UPROPERTY()
	TMap<FName, FRichCurve> CurveData;
};

USTRUCT()
struct FCacheSpawnableTemplate
{
	GENERATED_BODY()

	FCacheSpawnableTemplate()
		: DuplicatedTemplate(nullptr)
		, InitialTransform(FTransform::Identity)
		, ComponentTransform(FTransform::Identity)
	{}

	UPROPERTY(VisibleAnywhere, Category = "Caching")
	TObjectPtr<UObject> DuplicatedTemplate;

	UPROPERTY(VisibleAnywhere, Category = "Caching")
	FTransform InitialTransform;

	UPROPERTY(VisibleAnywhere, Category = "Caching")
	FTransform ComponentTransform;
};

struct FPlaybackTickRecord
{
	FPlaybackTickRecord()
		: CurrentDt(0.0f)
		, LastTime(0.0f)
		, SpaceTransform(FTransform::Identity)
	{
	}

	void Reset()
	{
		LastTime = 0.0f;
		LastEventPerTrack.Reset();
	}

	void SetLastTime(float InTime)
	{
		LastTime = InTime;
	}

	float GetTime() const
	{
		return LastTime + CurrentDt;
	}

	void SetDt(float NewDt)
	{
		CurrentDt = NewDt;
	}

	void SetSpaceTransform(const FTransform& InTransform)
	{
		SpaceTransform = InTransform;
	}

	const FTransform& GetSpaceTransform() const
	{
		return SpaceTransform;
	}

private:
	friend class UChaosCache;
	float              CurrentDt;
	float              LastTime;
	TMap<FName, int32> LastEventPerTrack;
	FTransform         SpaceTransform;
};

struct FCacheEvaluationContext
{
	FCacheEvaluationContext() = delete;
	explicit FCacheEvaluationContext(FPlaybackTickRecord& InRecord)
		: TickRecord(InRecord)
		, bEvaluateTransform(false)
		, bEvaluateCurves(false)
		, bEvaluateEvents(false)
		, bEvaluateNamedTransforms(false)
	{
	}

	FPlaybackTickRecord& TickRecord;
	bool                 bEvaluateTransform;
	bool                 bEvaluateCurves;
	bool                 bEvaluateEvents;
	TArray<int32>        EvaluationIndices;
	bool                 bEvaluateChannels;
	bool                 bEvaluateNamedTransforms;
};

struct FCacheEvaluationResult
{
public:
	float                                  EvaluatedTime;
	TArray<int32>                          ParticleIndices;
	TArray<FTransform>                     Transform;
	TArray<TMap<FName, float>>             Curves;
	TMap<FName, TArray<FCacheEventHandle>> Events;
	TMap<FName, TArray<float>>             Channels;
	TMap<FName, FTransform>                NamedTransforms;
};

struct FPendingParticleWrite
{
	int32                       ParticleIndex;
	FTransform                  PendingTransform;
	bool						bPendingDeactivate = false;
	TArray<TPair<FName, float>> PendingCurveData;
};

USTRUCT()
struct FRichCurves
{
	GENERATED_BODY()
	
	UPROPERTY()
	TArray<FRichCurve> RichCurves;
};

USTRUCT()
struct FCompressedRichCurves
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FCompressedRichCurve> CompressedRichCurves;
};

struct FPendingFrameWrite
{
	float                         Time;
	TArray<FPendingParticleWrite> PendingParticleData;
	TArray<TPair<FName, float>>   PendingCurveData;
	TMap<FName, FCacheEventTrack> PendingEvents;

	TArray<int32>				  PendingChannelsIndices;
	TMap<FName, TArray<float>>	  PendingChannelsData;

	TMap<FName, FTransform>       PendingNamedTransformData;

	template<typename T>
	FCacheEventTrack& FindOrAddEventTrack(FName InName)
	{
		// All event data must derive FCacheEventBase to be safely stored generically
		check(T::StaticStruct()->IsChildOf(FCacheEventBase::StaticStruct()));

		FCacheEventTrack* Existing = PendingEvents.Find(InName);

		return Existing ? *Existing : PendingEvents.Add(TTuple<FName, FCacheEventTrack>(InName, FCacheEventTrack(InName, T::StaticStruct())));
	}

	template<typename T>
	void PushEvent(FName InName, float InTime, const T& InEventStruct)
	{
		FCacheEventTrack& Track = FindOrAddEventTrack<T>(InName);
		Track.PushEvent<T>(InTime, InEventStruct);
	}
};

/**
 * A type that only the Chaos Cache is capable of constructing, passed back from TryRecord and TryPlayback to ensure the user is permitted to use the cache
 * This is also passed back to the EndPlayback and EndRecord functions to ensure that the caller has a valid token for the cache.
 */
class UChaosCache;
struct FCacheUserToken
{
	bool IsOpen() const
	{
		return bIsOpen && Owner;
	}

	// Allow moving, invalidating the old token
	FCacheUserToken(FCacheUserToken&& Other)
	{
		bIsOpen   = Other.bIsOpen;
		bIsRecord = Other.bIsRecord;
		Owner     = Other.Owner;

		Other.bIsOpen   = false;
		Other.bIsRecord = false;
		Other.Owner     = nullptr;
	}

private:
	friend UChaosCache;

	bool         bIsOpen;
	bool         bIsRecord;
	const UChaosCache* Owner;

	explicit FCacheUserToken(bool bInOpen, bool bInRecord, const UChaosCache* InOwner)
		: bIsOpen(bInOpen)
		, bIsRecord(bInRecord)
		, Owner(InOwner)
	{
	}

	FCacheUserToken()                       = delete;
	FCacheUserToken(const FCacheUserToken&) = delete;
	FCacheUserToken& operator=(const FCacheUserToken&) = delete;
	FCacheUserToken& operator=(FCacheUserToken&&) = delete;
};

/** Interface for the chaos cache data storage. @todo(chaos) : We should probably refactor the chaos cache to allow more custom storage */
UINTERFACE(MinimalAPI, Blueprintable)
class UChaosCacheData : public UInterface
{
	GENERATED_BODY()
};
class IChaosCacheData
{
	GENERATED_BODY()
 
public:
	/** Duration of the chaos cache */
	virtual float GetDuration() const = 0;
};

UCLASS(Experimental, MinimalAPI)
class UChaosCache : public UObject
{
	GENERATED_BODY()
public:

	CHAOSCACHING_API UChaosCache();

	//~ UObject interface
	CHAOSCACHING_API virtual void Serialize(FArchive& Ar) override;
	CHAOSCACHING_API virtual void PostLoad() override;
	//~ UObject interface END

	/**
	 * As we record post-simulate of physics, we're almost always taking data from a non-main thread (physics thread).
	 * Because of this we can't directly write into the cache, but instead into a pending frame queue that needs to be
	 * flushed on the main thread to write the pending data into the final storage.
	 */
	CHAOSCACHING_API void FlushPendingFrames();

	/**
	 * Reset and initialize a cache to make it ready to record the specified component
	 * @param InComponent Component to prepare the cache for
	 */
	CHAOSCACHING_API FCacheUserToken BeginRecord(const UPrimitiveComponent* InComponent, FGuid InAdapterId, const FTransform& SpaceTransform);

	/**
	 * End the recording session for the cache. At this point the cache is deemed to now contain
	 * all of the required data from the recording session and can then be post-processed and
	 * optimized which may involve key elimination and compression into a final format for runtime
	 * @param InOutToken The token that was given by BeginRecord
	 */
	CHAOSCACHING_API void EndRecord(FCacheUserToken& InOutToken);

	/**
	 * Initialise the cache for playback, may not take any actual action on the cache but
	 * will provide the caller with a valid cache user token if it is safe to continue with playback
	 */
	CHAOSCACHING_API FCacheUserToken BeginPlayback() const;

	/**
	 * End a playback session for the cache. There can be multiple playback sessions open for a
	 * cache as long as there isn't a recording session. Calling EndPlayback with a valid open
	 * token will decrease the session count.
	 * @param InOutToken The token that was given by BeginRecord
	 */
	CHAOSCACHING_API void EndPlayback(FCacheUserToken& InOutToken) const;

	/**
	 * Adds a new frame to process to a threadsafe queue for later processing in FlushPendingFrames
	 * @param InFrame New frame to accept, moved into the internal threadsafe queue
	 */
	CHAOSCACHING_API void AddFrame_Concurrent(FPendingFrameWrite&& InFrame);

	/**
	 * Gets the recorded duration of the cache
	 */
	CHAOSCACHING_API float GetDuration() const;

	/**
	 * Evaluate the cache with the specified parameters, returning the evaluated results
	 * @param InContext evaluation context
	 * @param MassToLocalTransforms MassToLocal trasnform if available ( geometry collection are using them )
	 * @see FCacheEvaluationContext
	 */
	CHAOSCACHING_API FCacheEvaluationResult Evaluate(const FCacheEvaluationContext& InContext, const TArray<FTransform>* MassToLocalTransforms) const;

	/**
	 * Initializes the spawnable template from a currently existing component so it can be spawned by the editor
	 * when a cache is dragged into the scene.
	 * @param InComponent Component to build the spawnable template from
	 */
	CHAOSCACHING_API void BuildSpawnableFromComponent(const UPrimitiveComponent* InComponent, const FTransform& SpaceTransform);

	/**
	 * Read access to the spawnable template stored in the cache
	 */
	CHAOSCACHING_API const FCacheSpawnableTemplate& GetSpawnableTemplate() const;

	/** Get the cache data asset if defined */
	IChaosCacheData* GetCacheData() const {return CacheData.GetInterface();}

	/** Set the cache data asset interface */
	void SetCacheData(IChaosCacheData* InCacheData) {CacheData.SetObject(CastChecked<UObject>(InCacheData)); CacheData.SetInterface(InCacheData);}

	/**
	 * Evaluates a single particle from the tracks array
	 * @param InIndex Particle track index (unchecked, ensure valid before call)
	 * @param InTickRecord Tick record for this evaluation
	 * @param MassToLocal MassToLocal transform, necessary for proper interpolation, if not available, nullptr shoudl used instead
	 * @param OutOptTransform Transform to fill, skipped if null
	 * @param OutOptCurves Curves to fill, skipped if null
	 */
	CHAOSCACHING_API void EvaluateSingle(int32 InIndex, FPlaybackTickRecord& InTickRecord, const FTransform* MassToLocal, FTransform* OutOptTransform, TMap<FName, float>* OutOptCurves) const;

	CHAOSCACHING_API void EvaluateTransform(const FPerParticleCacheData& InData, float InTime, const FTransform* MassToLocal, FTransform& OutTransform) const;
	CHAOSCACHING_API void EvaluateCurves(const FPerParticleCacheData& InData, float InTime, TMap<FName, float>& OutCurves) const;
	CHAOSCACHING_API void EvaluateEvents(FPlaybackTickRecord& InTickRecord, TMap<FName, TArray<FCacheEventHandle>>& OutEvents) const;

	CHAOSCACHING_API void CompressChannelsData(float ErrorThreshold, float SampleRate);

	UPROPERTY(VisibleAnywhere, Category = "Caching")
	float RecordedDuration;

	UPROPERTY(VisibleAnywhere, Category = "Caching")
	uint32 NumRecordedFrames;

	UPROPERTY(VisibleAnywhere, Category = "Caching")
	EChaosCacheInterpolationMode InterpolationMode = EChaosCacheInterpolationMode::QuatInterp;

	/** Maps a track index in the cache to the original particle index specified when recording */
	UPROPERTY()
	TArray<int32> TrackToParticle;

	/** Per-particle data, includes transforms, velocities and other per-particle, per-frame data */
	UPROPERTY()
	TArray<FPerParticleCacheData> ParticleTracks;

	/** Map a curve index in the cache to the original particle index specified when recording */
	UPROPERTY()
	TArray<int32> ChannelCurveToParticle;

	/** Per-particle data,  continuous per-frame data */
	UPROPERTY()
	TMap<FName,FRichCurves> ChannelsTracks;

	UPROPERTY()
	TMap<FName, FCompressedRichCurves> CompressedChannelsTracks;

	/** Per component/cache curve data, any continuous data that isn't per-particle can be stored here */
	UPROPERTY()
	TMap<FName, FRichCurve> CurveData;

	/** Per component/cache transform data.*/
	typedef FParticleTransformTrack FNamedTransformTrack;
	UPROPERTY()
	TMap<FName, FParticleTransformTrack> NamedTransformTracks;

	UPROPERTY()
	bool bCompressChannels = false;

	UPROPERTY()
	float ChannelsCompressionErrorThreshold = 1e-5f;

	UPROPERTY()
	float ChannelsCompressionSampleRate = 1.f / 30.f;

	template<typename T>
	FCacheEventTrack& FindOrAddEventTrack(FName InName)
	{
		// All event data must derive FCacheEventBase to be safely stored generically
		check(T::StaticStruct().IsChildOf(FCacheEventBase::StaticStruct()));

		FCacheEventTrack* Existing = EventTracks.Find(InName);

		return Existing ? *Existing : EventTracks.Add(InName, FCacheEventTrack(InName, T::StaticStruct()));
	}

private:

	/** Optional cache data to store on the chaos cache */
	UPROPERTY()
	TScriptInterface<IChaosCacheData> CacheData;
	
	CHAOSCACHING_API void FlushPendingFrames_ChannelOnlyReservePass(TQueue<FPendingFrameWrite, EQueueMode::Spsc>& LocalPendingWrites,
		bool& bCanSimpleCopyChannelData);

	// @return Whether or not any particle data was written
	template<EQueueMode Mode>
	bool FlushPendingFrames_MainPass(TQueue<FPendingFrameWrite, Mode>& InPendingWrites, bool bCanSimpleCopyChannelData);

	void CompressTracks();

	friend class AChaosCacheManager;

	/** Timestamped generic event tracks */
	UPROPERTY()
	TMap<FName, FCacheEventTrack> EventTracks;

	/** Spawn template for an actor that can play this cache */
	UPROPERTY(VisibleAnywhere, Category = "Caching")
	FCacheSpawnableTemplate Spawnable;

	/** GUID identifier for the adapter that spawned this cache */
	UPROPERTY()
	FGuid AdapterGuid;

	/** Version for controlling conditioning of older caches to work with current system. Newly created caches should always be saved as CurrentVersion. */
	UPROPERTY()
	int32 Version;

	// Version 0 : Pre versioning.
	// Version 1 : Introduction of actor space transforms & removal of baked MassTOLocal transform in GeometryCollections.
	static constexpr int32 CurrentVersion = 1;

	/** Pending writes from all threads to be consumed on the game thread, triggered by the recording cache manager */
	TQueue<FPendingFrameWrite, EQueueMode::Mpsc> PendingWrites;

	/** Counts for current number of users, should only ever have one recorder, and if we do no playbacks */
	std::atomic<int32> CurrentRecordCount;
	mutable std::atomic<int32> CurrentPlaybackCount;

	/** Indicates that we need to strip MassToLocal before playing the cache. */
	bool bStripMassToLocal;

	/** Reverse Lookup for ChannelCurveToParticle. Rebuilt on load.*/
	TMap<int32,int32> ParticleToChannelCurve;

	/** Min time in case we are not writing to particle/curves/channels datas */
	float MinTime = TNumericLimits<float>::Max();

	/** Max time in case we are not writing to particle/curves/channels datas */
	float MaxTime = TNumericLimits<float>::Lowest();
};
