// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosCacheInterpolationMode.generated.h"

UENUM()
enum class EChaosCacheInterpolationMode : uint8
{
	/** Shortest Path or Quaternion interpolation for the rotation. */
	QuatInterp,

	/** Rotor or Euler Angle interpolation. */
	EulerInterp,

	/** Dual quaternion interpolation, follows helix or screw-motion path between keyframes.   */
	DualQuatInterp
};