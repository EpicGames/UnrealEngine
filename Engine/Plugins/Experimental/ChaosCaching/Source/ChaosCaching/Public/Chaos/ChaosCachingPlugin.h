// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

namespace Chaos {
	class FGeometryCollectionCacheAdapter;
	class FStaticMeshCacheAdapter;
	class FSkeletalMeshCacheAdapter;
}

DECLARE_LOG_CATEGORY_EXTERN(LogChaosCache, Verbose, All);

class IChaosCachingPlugin : public IModuleInterface
{
public:
	virtual void StartupModule();
	virtual void ShutdownModule();

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IChaosCachingPlugin& Get()
	{
		return FModuleManager::LoadModuleChecked<IChaosCachingPlugin>("ChaosCaching");
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("ChaosCaching");
	}

private:
	TUniquePtr<Chaos::FGeometryCollectionCacheAdapter> GeometryCollectionAdapter;
	TUniquePtr<Chaos::FStaticMeshCacheAdapter> StaticMeshAdapter;

	FDelegateHandle OnCreateMovieSceneObjectSpawnerHandle;
};
