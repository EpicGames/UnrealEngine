// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LevelSequenceActorSpawner.h"

/** Chaos cache Manager spawner to create a new cache manager from the spawnable template */
class FChaosCacheObjectSpawner : public FLevelSequenceActorSpawner
{
public:
	UE_DEPRECATED(5.5, "This has been deprecated as it is no longer necessary. We use UMovieSceneSpawnableChaosCache custom binding instead to handle this logic")
	CHAOSCACHING_API FChaosCacheObjectSpawner();

	/** Static method to create the object spawner */
	UE_DEPRECATED(5.5, "This has been deprecated as it is no longer necessary. We use UMovieSceneSpawnableChaosCache custom binding instead to handle this logic")
	static CHAOSCACHING_API TSharedRef<IMovieSceneObjectSpawner> CreateObjectSpawner();

	// IMovieSceneObjectSpawner interface
	UE_DEPRECATED(5.5, "This has been deprecated as it is no longer necessary. We use UMovieSceneSpawnableChaosCache custom binding instead to handle this logic")
	CHAOSCACHING_API virtual UClass* GetSupportedTemplateType() const override;

	UE_DEPRECATED(5.5, "This has been deprecated as it is no longer necessary. We use UMovieSceneSpawnableChaosCache custom binding instead to handle this logic")
	CHAOSCACHING_API virtual UObject* SpawnObject(FMovieSceneSpawnable& Spawnable, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) override;
	
	UE_DEPRECATED(5.5, "This has been deprecated as it is no longer necessary. We use UMovieSceneSpawnableChaosCache custom binding instead to handle this logic")
	CHAOSCACHING_API virtual void DestroySpawnedObject(UObject& Object) override;
	virtual bool IsEditor() const override { return true; }
	virtual int32 GetSpawnerPriority() const override { return 1; }
};
