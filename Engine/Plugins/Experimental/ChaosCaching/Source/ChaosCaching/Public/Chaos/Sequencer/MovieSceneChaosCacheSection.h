// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sections/MovieSceneBaseCacheSection.h"
#include "MovieSceneChaosCacheSection.generated.h"

USTRUCT()
struct FMovieSceneChaosCacheParams : public FMovieSceneBaseCacheParams
{
	GENERATED_BODY()

	CHAOSCACHING_API FMovieSceneChaosCacheParams();
	virtual ~FMovieSceneChaosCacheParams() override = default;

	/** Gets the animation sequence length, not modified by play rate */
	CHAOSCACHING_API virtual float GetSequenceLength() const override;
	
	/** The animation this section plays */
	UPROPERTY(EditAnywhere, Category = "ChaosCache")
	TObjectPtr<class UChaosCacheCollection> CacheCollection;
};

/**
 * Movie scene section that control ChaosCache playback
 */
UCLASS(MinimalAPI)
class UMovieSceneChaosCacheSection : public UMovieSceneBaseCacheSection
{
	GENERATED_UCLASS_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Animation", meta = (ShowOnlyInnerProperties))
	FMovieSceneChaosCacheParams Params;
};
