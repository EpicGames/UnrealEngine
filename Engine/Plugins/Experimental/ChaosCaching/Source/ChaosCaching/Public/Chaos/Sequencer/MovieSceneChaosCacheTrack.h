// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneNameableTrack.h"
#include "Compilation/IMovieSceneTrackTemplateProducer.h"
#include "MovieSceneChaosCacheTrack.generated.h"

/**
 * Handles animation of ChaosCache
 */
UCLASS(MinimalAPI)
class UMovieSceneChaosCacheTrack
	: public UMovieSceneNameableTrack
	, public IMovieSceneTrackTemplateProducer
{
	GENERATED_UCLASS_BODY()

public:

	/** Adds a new animation to this track */
	virtual UMovieSceneSection* AddNewAnimation(FFrameNumber KeyTime, class AChaosCacheManager* ChaosCache);

	/** Gets the animation sections at a certain time */
	TArray<UMovieSceneSection*> GetAnimSectionsAtTime(FFrameNumber Time);

public:

	// UMovieSceneTrack interface
	virtual void RemoveAllAnimationData() override;
	virtual bool HasSection(const UMovieSceneSection& Section) const override;
	virtual void AddSection(UMovieSceneSection& Section) override;
	virtual void RemoveSection(UMovieSceneSection& Section) override;
	virtual void RemoveSectionAt(int32 SectionIndex) override;
	virtual bool IsEmpty() const override;
	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;

	// ~IMovieSceneTrackTemplateProducer interface
	virtual FMovieSceneEvalTemplatePtr CreateTemplateForSection(const UMovieSceneSection& InSection) const override;

#if WITH_EDITORONLY_DATA
	virtual FText GetDefaultDisplayName() const override;
#endif

private:

	/** List of all animation sections */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> AnimationSections;
};
