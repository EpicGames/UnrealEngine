// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Bindings/MovieSceneSpawnableActorBinding.h"
#include "MovieSceneSpawnableChaosCacheBinding.generated.h"

/*
* An override of UMovieSceneSpawnableActorBinding that adds some custom behavior on Spawn
* 
*/
UCLASS(BlueprintType, MinimalAPI, EditInlineNew, DefaultToInstanced, Meta=(DisplayName="Spawnable Chaos Cache"))
class UMovieSceneSpawnableChaosCacheBinding
	: public UMovieSceneSpawnableActorBinding
{
public:

	GENERATED_BODY()

public:

	/* MovieSceneCustomBinding overrides*/
	bool SupportsBindingCreationFromObject(const UObject* SourceObject) const override;

#if WITH_EDITOR
	FText GetBindingTypePrettyName() const override;
#endif

	/* Overridden to handle Chaos Cache -specific spawning */
	UObject* SpawnObjectInternal(UWorld* WorldContext, FName SpawnName, const FGuid& BindingId, int32 BindingIndex, UMovieScene& MovieScene, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) override;

protected:

	// Ensures we override priority of UMovieSceneSpawnableActorBinding
	int32 GetCustomBindingPriority() const override { return BaseCustomPriority; }

};

