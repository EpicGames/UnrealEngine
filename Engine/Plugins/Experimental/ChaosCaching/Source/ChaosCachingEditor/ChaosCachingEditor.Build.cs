// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ChaosCachingEditor : ModuleRules
	{
        public ChaosCachingEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Slate",
					"SlateCore",
					"InputCore",
					"Engine",
					"UnrealEd",
					"PropertyEditor",
					"BlueprintGraph",
					"ToolMenus",
					"PhysicsCore",
					"ChaosCaching",
					"GeometryCollectionEngine",
					"LevelEditor",
					"LevelSequence",
                    "MovieScene",
                    "MovieSceneTools",
                    "MovieSceneTracks",
                    "SequencerCore",
                    "Sequencer",
                    "TimeManagement",
					"EditorFramework",
					"TakesCore",
					"TakeRecorder",
					"SceneOutliner",
					"TakeTrackRecorders",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetDefinition"
				});

			SetupModulePhysicsSupport(Target);
			PrivateDefinitions.Add("CHAOS_INCLUDE_LEVEL_1=1");
		}
	}
}
