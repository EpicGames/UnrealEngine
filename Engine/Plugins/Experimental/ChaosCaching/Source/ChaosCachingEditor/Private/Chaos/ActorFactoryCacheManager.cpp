// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/ActorFactoryCacheManager.h"
#include "AssetRegistry/AssetData.h"
#include "Chaos/CacheCollection.h"
#include "Chaos/CacheManagerActor.h"
#include "Chaos/ChaosCache.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionSimulationTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorFactoryCacheManager)

#define LOCTEXT_NAMESPACE "CacheManagerActorFactory"

UActorFactoryCacheManager::UActorFactoryCacheManager()
{
	DisplayName            = LOCTEXT("DisplayName", "Chaos Cache Manager");
	NewActorClass          = AChaosCachePlayer::StaticClass();
	bUseSurfaceOrientation = false;
}

bool UActorFactoryCacheManager::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if (!AssetData.IsValid() || !AssetData.IsInstanceOf(UChaosCacheCollection::StaticClass()))
	{
		OutErrorMsg = LOCTEXT("NoCollection", "A valid cache collection must be specified.");
		return false;
	}

	return true;
}

void UActorFactoryCacheManager::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	AChaosCachePlayer*    Manager    = Cast<AChaosCachePlayer>(NewActor);
	UChaosCacheCollection* Collection = Cast<UChaosCacheCollection>(Asset);
	
	// The cachemanager exists now, start adding our spawnables
	if (Manager && Collection)
	{
		Manager->CacheCollection = Collection;

		if (!Manager->bIsEditorPreviewActor)
		{
			if (UWorld* World = Manager->GetWorld())
			{
				// Remove any pre-existing components
				Manager->ClearObservedComponents();
				
				for (UChaosCache* Cache : Collection->GetCaches())
				{
					if (!Cache)
					{
						continue;
					}

					const FCacheSpawnableTemplate& Template = Cache->GetSpawnableTemplate();
					if (Template.DuplicatedTemplate)
					{
						check(Template.DuplicatedTemplate->GetClass()->IsChildOf(UPrimitiveComponent::StaticClass()));

						FObjectDuplicationParameters Parameters(Template.DuplicatedTemplate, Manager);
						Parameters.FlagMask &= ~RF_Transient;
						Parameters.FlagMask &= ~RF_DefaultSubObject;
						Parameters.FlagMask &= ~RF_Transactional;
						
						UPrimitiveComponent* NewComponent = CastChecked<UPrimitiveComponent>(StaticDuplicateObjectEx(Parameters));
						NewComponent->SetupAttachment(Manager->GetRootComponent());
						NewComponent->SetRelativeTransform(Template.ComponentTransform);
						Manager->AddInstanceComponent(NewComponent);
						NewComponent->RegisterComponent();
						
						if (UGeometryCollectionComponent* GCComp = Cast<UGeometryCollectionComponent>(NewComponent))
						{
							GCComp->ObjectType = EObjectStateTypeEnum::Chaos_Object_Kinematic;
						}

						NewComponent->BodyInstance.bSimulatePhysics = false;

						FObservedComponent& Observed = Manager->AddNewObservedComponent(NewComponent);
						Observed.CacheName = Cache->GetFName();
					}
				}
			}
		}
		// Caches placed in this fashion are for Playback, not recording.
		Manager->CacheMode = ECacheMode::Play;
		Manager->OnStartFrameChanged(0.0);
	}	
}

UObject* UActorFactoryCacheManager::GetAssetFromActorInstance(AActor* ActorInstance)
{
	if(AChaosCacheManager* Manager = Cast<AChaosCacheManager>(ActorInstance))
	{
		return Manager->CacheCollection;
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
