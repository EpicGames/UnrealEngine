// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinitionDefault.h"
#include "Chaos/CacheCollection.h"
#include "AssetDefinition_ChaosCacheCollection.generated.h"

UCLASS()
class UAssetDefinition_ChaosCacheCollection : public UAssetDefinitionDefault
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override
	{
		return NSLOCTEXT("AssetDefinition", "AssetDefinition_ChaosCacheCollection", "Chaos Cache Collection");
	}

	virtual FLinearColor GetAssetColor() const override
	{
		return FColor(255, 127, 40);
	}

	virtual TSoftClassPtr<UObject> GetAssetClass() const override
	{
		return UChaosCacheCollection::StaticClass();
	}

	virtual FText GetAssetDescription(const FAssetData& AssetData) const override
	{
		return NSLOCTEXT("AssetDefinition", "AssetDefinition_ChaosCacheCollectionDescription", "A collection of physically active component caches that can be used to record and replay Chaos simulation recordings.");
	}

	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const TArray<FAssetCategoryPath> Categories = 
			{
				FAssetCategoryPath(EAssetCategoryPaths::Physics, NSLOCTEXT("AssetDefinition", "AssetDefinition_ChaosCacheCollectionSubMenu", "Chaos"), ECategoryMenuType::Section)
			};
		return Categories;
	}
	// UAssetDefinition End
};
