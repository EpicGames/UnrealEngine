// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/CacheCollectionCustomization.h"
#include "Chaos/CacheCollection.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Chaos/ChaosCache.h"

#define LOCTEXT_NAMESPACE "CacheCollectionDetails"

TSharedRef<IDetailCustomization> FCacheCollectionDetails::MakeInstance()
{
	return MakeShareable(new FCacheCollectionDetails);
}

void FCacheCollectionDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);

	NameEditBoxes.Empty();

	// In the simple asset editor we should only get one object
	if(Objects.Num() != 1)
	{
		return;
	}

	Item = Objects[0];

	if (UChaosCacheCollection* Collection = GetSelectedCollection())
	{
		if (Collection->Caches.Num())
		{
			for (int i=0;i<Collection->Caches.Num();i++)
			{
				NameEditBoxes.Add(i, TSharedPtr<SEditableTextBox>());
			}

			TSharedRef<IPropertyHandle> CachesProp = DetailBuilder.GetProperty("Caches");
			IDetailCategoryBuilder& CacheCategory = DetailBuilder.EditCategory(CachesProp->GetDefaultCategoryName());

			TSharedRef<FDetailArrayBuilder> CacheArrayBuilder = MakeShareable(new FDetailArrayBuilder(CachesProp, true, false, true));
			CacheArrayBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateSP(this, &FCacheCollectionDetails::GenerateCacheArrayElementWidget, &DetailBuilder));
			CacheCategory.AddCustomBuilder(CacheArrayBuilder);
		}
	}
}

const UChaosCache* FCacheCollectionDetails::GetCacheCollection(int32 Index) const
{
	if (const UChaosCacheCollection* Collection = GetSelectedCollection())
	{
		if(0 <= Index && Index < Collection->Caches.Num())
		{
			return Collection->Caches[Index];
		}
	}
	return nullptr;
}


UChaosCacheCollection* FCacheCollectionDetails::GetSelectedCollection()
{
	if(UObject* ItemObj = Item.Get())
	{
		if(UChaosCacheCollection* Collection = Cast<UChaosCacheCollection>(ItemObj))
		{
			return Collection;
		}
	}

	return nullptr;
}

const UChaosCacheCollection* FCacheCollectionDetails::GetSelectedCollection() const
{
	if(const UObject* ItemObj = Item.Get())
	{
		if(const UChaosCacheCollection* Collection = Cast<UChaosCacheCollection>(ItemObj))
		{
			return Collection;
		}
	}

	return nullptr;
}

FText FCacheCollectionDetails::GetCacheName(int32 InCacheIndex) const
{
	if(const UChaosCacheCollection* Collection = GetSelectedCollection())
	{
		if(Collection->Caches.IsValidIndex(InCacheIndex) && Collection->Caches[InCacheIndex])
		{
			return FText::FromString(Collection->Caches[InCacheIndex]->GetName());
		}
	}

	return LOCTEXT("InvalidObject", "Invalid");
}

void FCacheCollectionDetails::OnDeleteCache(int32 InArrayIndex, IDetailLayoutBuilder* InLayoutBuilder)
{
	if(UChaosCacheCollection* Collection = GetSelectedCollection())
	{
		if(Collection->Caches.IsValidIndex(InArrayIndex) )
		{
			Collection->Caches.RemoveAt(InArrayIndex);
			InLayoutBuilder->ForceRefreshDetails();
		}
	}
}

bool IsValidName(UChaosCacheCollection* Collection, FName InName)
{
	if(Collection)
	{
		UObject* Existing = StaticFindObject(UChaosCache::StaticClass(), Collection, *InName.ToString());
		return !Existing;
	}
	return false;
}

void FCacheCollectionDetails::OnChangeCacheName(const FText& InNewName, int32 InIndex)
{
	UChaosCacheCollection* Collection = GetSelectedCollection();

	if(!Collection)
	{
		return;
	}


	if (TSharedPtr<SEditableTextBox> TextBox = NameEditBoxes[InIndex])
	{
		if (!IsValidName(Collection, FName(InNewName.ToString())))
		{
			TextBox->SetError(LOCTEXT("InvalidNameError", "Invalid Cache Name"));
		}
		else
		{
			TextBox->SetError(TEXT(""));
		}
	}
}

void FCacheCollectionDetails::OnCommitCacheName(const FText& InNewName, ETextCommit::Type InTextCommit, int32 InIndex)
{
	if(InTextCommit != ETextCommit::OnEnter)
	{
		// Focus was taken away, don't perform the edit
		return;
	}

	if(UChaosCacheCollection* Collection = GetSelectedCollection())
	{
		FName NewName(*InNewName.ToString());
		if(IsValidName(Collection, NewName))
		{
			UChaosCache* CurrCache = Collection->Caches[InIndex];
			if (CurrCache)
			{
				CurrCache->Rename(*NewName.ToString());
			}
		}
	}
}

void FCacheCollectionDetails::GenerateCacheArrayElementWidget(TSharedRef<IPropertyHandle> InPropertyHandle, int32 ArrayIndex, IDetailChildrenBuilder& ChildrenBuilder, IDetailLayoutBuilder* DetailLayout)
{
	IDetailPropertyRow& PropRow = ChildrenBuilder.AddProperty(InPropertyHandle);
	PropRow.ShowPropertyButtons(false);
	PropRow.OverrideResetToDefault(FResetToDefaultOverride::Hide());

	FDetailWidgetRow& WidgetRow = PropRow.CustomWidget(true);

	WidgetRow.NameContent()
		[
			InPropertyHandle->CreatePropertyNameWidget()
		];

	TSharedPtr<SEditableTextBox> TextBox = SNew(SEditableTextBox);
	if (NameEditBoxes.Contains(ArrayIndex))
	{
		TextBox = NameEditBoxes[ArrayIndex];
	}

	WidgetRow.ValueContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			.AutoWidth()
			[
				SAssignNew(TextBox, SEditableTextBox)
				.Text(this, &FCacheCollectionDetails::GetCacheName, ArrayIndex)
				.OnTextChanged(this, &FCacheCollectionDetails::OnChangeCacheName, ArrayIndex)
				.OnTextCommitted(this, &FCacheCollectionDetails::OnCommitCacheName, ArrayIndex)
			]

			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			.Padding(5,0,0,0)
			.AutoWidth()
			[
				PropertyCustomizationHelpers::MakeDeleteButton(FSimpleDelegate::CreateSP(this, &FCacheCollectionDetails::OnDeleteCache, ArrayIndex, DetailLayout))
			]
		];
}


#undef LOCTEXT_NAMESPACE
