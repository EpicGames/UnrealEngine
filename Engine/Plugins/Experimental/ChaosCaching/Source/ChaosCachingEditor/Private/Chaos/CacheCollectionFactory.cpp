// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/CacheCollectionFactory.h"

#include "Chaos/CacheCollection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CacheCollectionFactory)

UCacheCollectionFactory::UCacheCollectionFactory()
{
	SupportedClass = UChaosCacheCollection::StaticClass();
}

bool UCacheCollectionFactory::CanCreateNew() const
{
	return true;
}

bool UCacheCollectionFactory::FactoryCanImport(const FString& Filename)
{
	return false;
}

UObject* UCacheCollectionFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags,
												   UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UChaosCacheCollection>(InParent, InClass, InName, Flags);
}

bool UCacheCollectionFactory::ShouldShowInNewMenu() const
{
	return true;
}

bool UCacheCollectionFactory::ConfigureProperties()
{
	return true;
}
