// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/CacheEditorCommands.h"

#define LOCTEXT_NAMESPACE "CacheEditorCommands"

void FCachingEditorCommands::RegisterCommands()
{
	UI_COMMAND(CreateCacheManager, "Create Cache Manager", "Adds a cache manager to observe compatible components in the selection set.", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
