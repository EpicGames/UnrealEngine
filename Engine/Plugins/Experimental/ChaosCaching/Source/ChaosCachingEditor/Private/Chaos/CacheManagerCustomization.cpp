// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/CacheManagerCustomization.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Widgets/Input/SButton.h"
#include "Chaos/CacheManagerActor.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/Input/SComboBox.h"

#define LOCTEXT_NAMESPACE "CacheManagerDetails"

FReply OnClickResetTransforms(TArray<AChaosCacheManager*> Managers)
{
	for(AChaosCacheManager* Manager : Managers)
	{
		if(Manager)
		{
			Manager->ResetAllComponentTransforms();
		}
	}

	return FReply::Handled();
}

TSharedRef<IDetailCustomization> FCacheManagerDetails::MakeInstance()
{
	return MakeShareable(new FCacheManagerDetails);
}

void FCacheManagerDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> Selected = DetailBuilder.GetSelectedObjects();
	TArray<AChaosCacheManager*> CacheManagersInSelection;

	bool bCanRecord = true;

	for(TWeakObjectPtr<UObject> Ptr : Selected)
	{
		if(AChaosCacheManager* Manager = Cast<AChaosCacheManager>(Ptr.Get()))
		{
			CacheManagersInSelection.Add(Manager);
			bCanRecord &= Manager->CanRecord();
		}
	}

	if(CacheManagersInSelection.Num() == 0)
	{
		return;
	}

	SetCacheModeOptions(bCanRecord);

	// Hide the Record mode
	CacheModeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(AChaosCacheManager, CacheMode), AChaosCacheManager::StaticClass());
	check(CacheModeHandle->IsValidHandle());

	DetailBuilder.EditDefaultProperty(CacheModeHandle)->CustomWidget()
		.NameContent()
		[
			CacheModeHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MaxDesiredWidth(250.0f)			
		[
			SNew(SComboBox<TSharedPtr<FText>>)
			.OptionsSource(&CacheModeComboList)
			.OnGenerateWidget(this, &FCacheManagerDetails::GenerateCacheModeWidget)
			.OnSelectionChanged(this, &FCacheManagerDetails::OnCacheModeChanged)
			.InitiallySelectedItem(GetCurrentCacheMode())	
			[
				SNew(STextBlock)
				.Text(this, &FCacheManagerDetails::GetCacheModeComboBoxContent)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.ColorAndOpacity(this, &FCacheManagerDetails::GetCacheModeTextColor)
				.ToolTipText(this, &FCacheManagerDetails::GetCacheModeComboBoxContent)
			] 
		];

	
	IDetailCategoryBuilder& CachingCategory = DetailBuilder.EditCategory("Caching");
	FDetailWidgetRow& ResetPositionsRow = CachingCategory.AddCustomRow(FText::GetEmpty());

	ResetPositionsRow.ValueContent()
	.MinDesiredWidth(300.0f)
	[
		SNew(SButton)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.OnClicked_Static(&OnClickResetTransforms, CacheManagersInSelection)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ResetPositions", "Reset All Component Transforms"))
		]
	];
}

void FCacheManagerDetails::SetCacheModeOptions(bool bRecord)
{
	CacheModeComboList.Empty(3);
	CacheModeColor.Empty(3);

	CacheModeComboList.Add(MakeShareable(new FText(LOCTEXT("StaticPose","Static Pose"))));
	CacheModeColor.Add(FSlateColor::UseForeground());

	CacheModeComboList.Add(MakeShareable(new FText(LOCTEXT("Play","Play"))));
	CacheModeColor.Add(FSlateColor::UseForeground());

	if (bRecord)
	{
		CacheModeComboList.Add(MakeShareable(new FText(LOCTEXT("Record", "RECORD"))));
		CacheModeColor.Add(FLinearColor::Red);
	}
}

void FCacheManagerDetails::GenerateCacheArrayElementWidget(TSharedRef<IPropertyHandle> PropertyHandle, int32 ArrayIndex, IDetailChildrenBuilder& ChildrenBuilder, IDetailLayoutBuilder* DetailLayout)
{
}

TSharedRef<SWidget> FCacheManagerDetails::GenerateCacheModeWidget(TSharedPtr<FText> InItem)
{
	FSlateColor ItemColor = FSlateColor::UseForeground();
	if (InItem->EqualTo(LOCTEXT("Record","RECORD"), ETextComparisonLevel::Default))
	{
		ItemColor = FLinearColor::Red;
	}

	return
		SNew(STextBlock)
		.Text(*InItem)
		.ColorAndOpacity(ItemColor)
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

void FCacheManagerDetails::OnCacheModeChanged(TSharedPtr<FText> NewSelection, ESelectInfo::Type SelectInfo)
{
	FText const NewCacheModeName = *NewSelection.Get();
	for (int32 Idx = 0; Idx < CacheModeComboList.Num(); ++Idx)
	{
		if (NewCacheModeName.EqualTo(*CacheModeComboList[Idx].Get(), ETextComparisonLevel::Default))
		{
			CacheModeHandle->SetValue(static_cast<uint8>(Idx));
			return;
		}
	}
	
	// should not get here
	check(false);
}

FText FCacheManagerDetails::GetCacheModeComboBoxContent() const
{
	return *CacheModeComboList[GetClampedCacheModeIndex()].Get();
}

TSharedPtr<FText> FCacheManagerDetails::GetCurrentCacheMode() const
{
	return CacheModeComboList[GetClampedCacheModeIndex()];
}

FSlateColor FCacheManagerDetails::GetCacheModeTextColor() const
{
	return CacheModeColor[GetClampedCacheModeIndex()];
}

int32  FCacheManagerDetails::GetClampedCacheModeIndex() const
{
	// A CacheManager may have a CacheMode (ie. Record) that we won't provide option for,
	// since other Managers without that option may also be selected.
	uint8 CacheMode;
	CacheModeHandle->GetValue(CacheMode);

	return FMath::Min(static_cast<int32>(CacheMode), CacheModeComboList.Num() - 1);
}


TSharedRef<IPropertyTypeCustomization> FObservedComponentDetails::MakeInstance()
{
	return MakeShareable(new FObservedComponentDetails);
}

void FObservedComponentDetails::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	HeaderRow
		.NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		[
			PropertyHandle->CreatePropertyValueWidget(/*bDisplayDefaultPropertyButtons =*/false)
		];
}

FReply OnClickResetSingleTransform(AChaosCacheManager* InManager, int32 InIndex)
{
	if(InManager)
	{
		InManager->ResetSingleTransform(InIndex);
	}

	return FReply::Handled();
}

FReply OnClickSelectComponent(AChaosCacheManager* InManager, int32 InIndex)
{
	if(InManager)
	{
		InManager->SelectComponent(InIndex);
	}

	return FReply::Handled();
}

void FObservedComponentDetails::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	uint32 NumChildren = 0;
	PropertyHandle->GetNumChildren(NumChildren);

	for(uint32 ChildNum = 0; ChildNum < NumChildren; ++ChildNum)
	{
		ChildBuilder.AddProperty(PropertyHandle->GetChildHandle(ChildNum).ToSharedRef());
	}

	TArray<TWeakObjectPtr<UObject>> SelectedObjects = ChildBuilder.GetParentCategory().GetParentLayout().GetSelectedObjects();
	if(SelectedObjects.Num() == 1)
	{
		if(AChaosCacheManager* SelectedManager = Cast<AChaosCacheManager>(SelectedObjects[0].Get()))
		{
			const int32 ArrayIndex = PropertyHandle->GetIndexInArray();

			ChildBuilder.AddCustomRow(FText::GetEmpty())
				.ValueContent()
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.Padding(0.0f, 0.0f, 0.0f, 3.0f)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.OnClicked_Static(&OnClickResetSingleTransform, SelectedManager, ArrayIndex)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ItemResetTransform", "Reset Transform"))
						]
					]
					+ SVerticalBox::Slot()
					.Padding(0.0f, 0.0f, 0.0f, 3.0f)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.OnClicked_Static(&OnClickSelectComponent, SelectedManager, ArrayIndex)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ItemSelect", "Select Component"))
						]
					]
				];
		}
	}
}

#undef LOCTEXT_NAMESPACE
