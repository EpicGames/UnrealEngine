// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/ChaosCachingEditorPlugin.h"
#include "Chaos/ChaosCachingEditorStyle.h"
#include "Chaos/Adapters/CacheAdapter.h"
#include "Chaos/CacheCollectionCustomization.h"
#include "Chaos/CacheEditorCommands.h"
#include "Chaos/CacheManagerActor.h"
#include "Chaos/CacheManagerCustomization.h"
#include "Chaos/CacheCollectionFactory.h"
#include "Chaos/CacheCollection.h"
#include "Components/PrimitiveComponent.h"
#include "Features/IModularFeatures.h"
#include "ISceneOutliner.h"
#include "Sequencer/TakeRecorderChaosCacheSource.h"
#include "PropertyEditorModule.h"
#include "LevelEditor.h"
#include "ISequencerModule.h"
#include "ScopedTransaction.h"
#include "Sequencer/ChaosCacheTrackEditor.h"
#include "ITakeRecorderModule.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ActorTreeItem.h"
#include "AssetToolsModule.h"
#include "SceneOutlinerModule.h"
#include "Selection.h"
#include "TakeRecorderSources.h"

IMPLEMENT_MODULE(IChaosCachingEditorPlugin, ChaosCachingEditor)

#define LOCTEXT_NAMESPACE "CacheEditorPlugin"

static void AddChaosCacheSource(UTakeRecorderSources* Sources,  const FName& SubjectName)
{
	FScopedTransaction Transaction(LOCTEXT("AddChaosCacheSource","Add Chaos Cache Source"));

	Sources->Modify();

	UTakeRecorderChaosCacheSource* NewSource = Sources->AddSource<UTakeRecorderChaosCacheSource>();
}

void AddChaosCacheSources(
	UTakeRecorderSources* TakeRecorderSources,
	TArrayView<AActor* const> ActorsToRecord)
{
	if (ActorsToRecord.Num() > 0)
	{
		FScopedTransaction Transaction(FText::Format(
			LOCTEXT("AddSources", "Add Recording {0}|plural(one=Source, other=Sources)"), ActorsToRecord.Num()));
		TakeRecorderSources->Modify();

		for (AActor* Actor : ActorsToRecord)
		{
			if (Actor->IsA<AChaosCacheManager>())
			{
				UTakeRecorderChaosCacheSource* NewSource = TakeRecorderSources->AddSource<UTakeRecorderChaosCacheSource>();
				NewSource->ChaosCacheManager = Cast<AChaosCacheManager>(Actor);

				// Send a PropertyChangedEvent so the class catches the callback and rebuilds the property map.
				FPropertyChangedEvent PropertyChangedEvent(UTakeRecorderChaosCacheSource::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTakeRecorderChaosCacheSource, ChaosCacheManager)), EPropertyChangeType::ValueSet);
				NewSource->PostEditChangeProperty(PropertyChangedEvent);
			}
		}
	}
}

static void PopulateChaosCacheSubMenu(FMenuBuilder& MenuBuilder, UTakeRecorderSources* Sources)
{
	TSet<const AActor*> CacheActors;
	
	for (UTakeRecorderSource* Source : Sources->GetSources())
	{
		const UTakeRecorderChaosCacheSource* ChaosCache   = Cast<UTakeRecorderChaosCacheSource>(Source);
		if (ChaosCache && ChaosCache->ChaosCacheManager.IsValid())
		{
			CacheActors.Add(ChaosCache->ChaosCacheManager.Get());
		}
	}
	
	auto OutlinerFilterPredicate = [InExistingActors = MoveTemp(CacheActors)](const AActor* InActor)
	{
		return !InExistingActors.Contains(InActor) && InActor->IsA<AChaosCacheManager>();
	};
	
	// Set up a menu entry to add the selected actor(s) to the sequencer
	TArray<AActor*> SelectedActors;
	GEditor->GetSelectedActors()->GetSelectedObjects(SelectedActors);
	SelectedActors.RemoveAll([&](const AActor* In){ return !OutlinerFilterPredicate(In); });
	
	FText SelectedLabel;
	if (SelectedActors.Num() == 1)
	{
		SelectedLabel = FText::Format(LOCTEXT("AddSpecificActor", "Add '{0}'"), FText::FromString(SelectedActors[0]->GetActorLabel()));
	}
	else if (SelectedActors.Num() > 1)
	{
		SelectedLabel = FText::Format(LOCTEXT("AddCurrentActorSelection", "Add Current Selection ({0} actors)"), FText::AsNumber(SelectedActors.Num()));
	}
	
	if (!SelectedLabel.IsEmpty())
	{
		MenuBuilder.AddMenuEntry(
			SelectedLabel,
			FText(),
			FSlateIcon(),
			FExecuteAction::CreateLambda([Sources, SelectedActors]{
				AddChaosCacheSources(Sources, SelectedActors);
			})
		);
	}
	
	MenuBuilder.BeginSection("ChooseActorSection", LOCTEXT("ChooseActor", "Choose Actor:"));
	{
		// Set up a menu entry to add any arbitrary actor to the sequencer
		FSceneOutlinerInitializationOptions InitOptions;
		{
			// We hide the header row to keep the UI compact.
			InitOptions.bShowHeaderRow = false;
			InitOptions.bShowSearchBox = true;
			InitOptions.bShowCreateNewFolder = false;
			InitOptions.bFocusSearchBoxWhenOpened = true;
	
			// Only want the actor label column
			InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Label(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 0));
	
			// Only display actors that are not possessed already
			InitOptions.Filters->AddFilterPredicate<FActorTreeItem>(FActorTreeItem::FFilterPredicate::CreateLambda(OutlinerFilterPredicate));
		}
	
		// actor selector to allow the user to choose an actor
		FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");
		TSharedRef< SWidget > MiniSceneOutliner =
			SNew(SBox)
			.MaxDesiredHeight(400.0f)
			.WidthOverride(300.0f)
			[
				SceneOutlinerModule.CreateActorPicker(
					InitOptions,
					FOnActorPicked::CreateLambda([Sources](AActor* Actor){
						// Create a new binding for this actor
						FSlateApplication::Get().DismissAllMenus();
						AddChaosCacheSources(Sources, MakeArrayView(&Actor, 1));
					})
				)
			];
	
		MenuBuilder.AddWidget(MiniSceneOutliner, FText::GetEmpty(), true);
	}
	MenuBuilder.EndSection();
}

static void PopulateSourcesMenu(FMenuBuilder& MenuBuilder, UTakeRecorderSources* Sources)
{
	FName ExtensionName = "ChaosCacheSourceSubMenu";

	MenuBuilder.AddSubMenu(
		NSLOCTEXT("TakeRecorderSources", "ChaosCacheList_Label", "From ChaosCache"),
		NSLOCTEXT("TakeRecorderSources", "ChaosCacheList_Tip", "Add a new recording source from a Chaos Cache Subject"),
		FNewMenuDelegate::CreateStatic(PopulateChaosCacheSubMenu, Sources),
		FUIAction(),
		ExtensionName,
		EUserInterfaceActionType::Button
	);
}

static void ExtendSourcesMenu(TSharedRef<FExtender> Extender, UTakeRecorderSources* Sources)
{
	Extender->AddMenuExtension("Sources", EExtensionHook::Before, nullptr, FMenuExtensionDelegate::CreateStatic(PopulateSourcesMenu, Sources));
}

static const FName MovieSceneTrackRecorderFactoryName("MovieSceneTrackRecorderFactory");

void IChaosCachingEditorPlugin::StartupModule()
{
	FCachingEditorCommands::Register();

	// Register level editor menu extender
	FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors LevelEditorMenuExtenderDelegate = FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors::CreateRaw(this, &IChaosCachingEditorPlugin::ExtendLevelViewportContextMenu);
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
	auto& MenuExtenders = LevelEditorModule.GetAllLevelViewportContextMenuExtenders();
	MenuExtenders.Add(LevelEditorMenuExtenderDelegate);
	StartupHandle = MenuExtenders.Last().GetHandle();

	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomClassLayout("ChaosCacheCollection", FOnGetDetailCustomizationInstance::CreateStatic(&FCacheCollectionDetails::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("ChaosCacheManager", FOnGetDetailCustomizationInstance::CreateStatic(&FCacheManagerDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("ObservedComponent", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FObservedComponentDetails::MakeInstance));
	
	ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
	TrackEditorBindingHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FChaosCacheTrackEditor::CreateTrackEditor));
	
	ITakeRecorderModule& TakeRecorderModule = FModuleManager::Get().LoadModuleChecked<ITakeRecorderModule>("TakeRecorder");
	SourcesMenuExtension = TakeRecorderModule.RegisterSourcesMenuExtension(FOnExtendSourcesMenu::CreateStatic(ExtendSourcesMenu));
	
	IModularFeatures::Get().RegisterModularFeature(MovieSceneTrackRecorderFactoryName, &MovieSceneChaosCacheTrackRecorder);
}

void IChaosCachingEditorPlugin::ShutdownModule()
{
	if(UObjectInitialized())
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomPropertyTypeLayout("ObservedComponent");
		PropertyModule.UnregisterCustomClassLayout("ChaosCacheManager");
		PropertyModule.UnregisterCustomClassLayout("ChaosCacheCollection");

		// Unregister level editor menu extender
		if (FModuleManager::Get().IsModuleLoaded(TEXT("LevelEditor")))
		{
			FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
			LevelEditorModule.GetAllLevelViewportContextMenuExtenders().RemoveAll([&](const FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors& Delegate) {
				return Delegate.GetHandle() == StartupHandle;
				});
		}

		FCachingEditorCommands::Unregister();

		ISequencerModule* SequencerModulePtr = FModuleManager::Get().GetModulePtr<ISequencerModule>("Sequencer");
		if (SequencerModulePtr)
		{
			SequencerModulePtr->UnRegisterTrackEditor(TrackEditorBindingHandle);
		}

		ITakeRecorderModule* TakeRecorderModule = FModuleManager::Get().GetModulePtr<ITakeRecorderModule>("TakeRecorder");
		if (TakeRecorderModule)
		{
			TakeRecorderModule->UnregisterSourcesMenuExtension(SourcesMenuExtension);
		}

		IModularFeatures::Get().UnregisterModularFeature(MovieSceneTrackRecorderFactoryName, &MovieSceneChaosCacheTrackRecorder);
	}
}

bool IsCreateCacheManagerVisible();
bool IsSetAllRecordVisible();
bool IsSetAllPlayVisible();

TSharedRef<FExtender> IChaosCachingEditorPlugin::ExtendLevelViewportContextMenu(const TSharedRef<FUICommandList> InCommandList, const TArray<AActor*> SelectedActors)
{
	TSharedRef<FExtender> Extender(new FExtender());

	Extender->AddMenuExtension("ActorUETools", EExtensionHook::After, InCommandList, FMenuExtensionDelegate::CreateLambda(
		[this, SelectedActors](FMenuBuilder& MenuBuilder)
		{
			MenuBuilder.AddSubMenu(
				LOCTEXT("ChaosSectionLabel", "Chaos"),
				LOCTEXT("Tooltip_Caching", "Options for manipulating cache managers and their observed components"),
				FNewMenuDelegate::CreateLambda([this](FMenuBuilder &InMenuBuilder) {
					RegisterCachingSubMenu(InMenuBuilder);
					}),
				FUIAction(FExecuteAction(), FCanExecuteAction::CreateLambda([]() -> bool
					{
						return IsCreateCacheManagerVisible() || IsSetAllPlayVisible() || IsSetAllRecordVisible();
					})),
					NAME_None,
					EUserInterfaceActionType::Button,
					false,
					FSlateIcon(FChaosCachingEditorStyle::Get().GetStyleSetName(), "ChaosCachingEditor.Fracture"));		
		}
	));

	return Extender;
}


void IChaosCachingEditorPlugin::RegisterCachingSubMenu(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.BeginSection("Caching", LOCTEXT("SubMenu_Caching", "Caching"));

	InMenuBuilder.AddMenuEntry(
							LOCTEXT("MenuItem_CreateCacheManager", "Create Cache Manager"),
							LOCTEXT("MenuItem_CreateCacheManager_ToolTip", "Adds a cache manager to observe compatible components in the selection set."),
							FSlateIcon(),
							FUIAction(FExecuteAction::CreateRaw(this, &IChaosCachingEditorPlugin::OnCreateCacheManager),
									  FCanExecuteAction(),
									  FIsActionChecked(),
									  FIsActionButtonVisible::CreateStatic(&IsCreateCacheManagerVisible)));
	InMenuBuilder.EndSection();
}

void IChaosCachingEditorPlugin::OnCreateCacheManager()
{
	FScopedTransaction Transaction(LOCTEXT("ChaosCachingEditorPlugin_CreateCacheManager_Transaction", "Create a Chaos Cache Manager From Selection"));

	auto SpawnManager = [](UWorld* InWorld) -> AChaosCacheManager* {
		FActorSpawnParameters Params;
		return InWorld->SpawnActor<AChaosCacheManager>();
	};

	AChaosCacheManager* Manager = nullptr;

	// Get the implementation of our adapters for identifying compatible components
	IModularFeatures&                      ModularFeatures = IModularFeatures::Get();
	TArray<Chaos::FComponentCacheAdapter*> Adapters = ModularFeatures.GetModularFeatureImplementations<Chaos::FComponentCacheAdapter>(Chaos::FComponentCacheAdapter::FeatureName);

	USelection* SelectedActors = GEditor->GetSelectedActors();

	TArray<AActor*> Actors;
	SelectedActors->GetSelectedObjects<AActor>(Actors);

	TArray<UActorComponent*> ComponentArray;
	for(AActor* Actor : Actors)
	{
		ComponentArray.Reset();
		Actor->GetComponents(ComponentArray);

		for(UActorComponent* Component : ComponentArray)
		{
			if(Component->CreationMethod == EComponentCreationMethod::UserConstructionScript)
			{
				// Can't hold references to UCS created components (See FComponentEditorUtils::MakeComponentReference)
				continue;
			}

			if(UPrimitiveComponent* PrimitiveComp = Cast<UPrimitiveComponent>(Component))
			{
				Chaos::FComponentCacheAdapter* BestFitAdapter = Chaos::FAdapterUtil::GetBestAdapterForClass(PrimitiveComp->GetClass(), false);

				// Can't be observed
				if(!BestFitAdapter)
				{
					continue;
				}

				// If we get here without a manager, lazy spawn one
				if(!Manager)
				{
					Manager = SpawnManager(Component->GetWorld());
				}

				check(Manager);

				// we do not want to record transient component as they may not exists when we start recording 
				if (PrimitiveComp->HasAnyFlags(RF_Transient))
				{
					continue;
				}

				const FObservedComponent* Existing = Manager->FindObservedComponent(PrimitiveComp);

				if(!Existing)
				{
					FObservedComponent& NewEntry = Manager->AddNewObservedComponent(PrimitiveComp);
					NewEntry.bIsSimulating = PrimitiveComp->BodyInstance.bSimulatePhysics;
				}
			}
		}
	}

	if (Manager)
	{
		// Create an associated Cache Collection
		if (Manager->ObservedComponents.Num() > 0)
		{
			FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
			UCacheCollectionFactory* Factory = NewObject<UCacheCollectionFactory>();
			UChaosCacheCollection* NewAsset = Cast<UChaosCacheCollection>(AssetToolsModule.Get().CreateAssetWithDialog(UChaosCacheCollection::StaticClass(), Factory));

			if (NewAsset)
			{
				Manager->CacheCollection = NewAsset;
			}
		}

		// Initialize observed components according to mode
		Manager->SetObservedComponentProperties(Manager->CacheMode);
	}
}

bool IsCreateCacheManagerVisible()
{
	IModularFeatures&                      ModularFeatures = IModularFeatures::Get();
	TArray<Chaos::FComponentCacheAdapter*> Adapters = ModularFeatures.GetModularFeatureImplementations<Chaos::FComponentCacheAdapter>(Chaos::FComponentCacheAdapter::FeatureName);

	USelection* SelectedActors = GEditor->GetSelectedActors();

	TArray<AActor*> Actors;
	SelectedActors->GetSelectedObjects<AActor>(Actors);

	TArray<UActorComponent*> ComponentArray;
	for(AActor* Actor : Actors)
	{
		ComponentArray.Reset();
		Actor->GetComponents(ComponentArray);

		for(UActorComponent* Component : ComponentArray)
		{
			if(Component->CreationMethod == EComponentCreationMethod::UserConstructionScript)
			{
				// Can't hold references to UCS created components (See FComponentEditorUtils::MakeComponentReference)
				continue;
			}

			if(UPrimitiveComponent* PrimitiveComp = Cast<UPrimitiveComponent>(Component))
			{
				Chaos::FComponentCacheAdapter* BestFitAdapter = Chaos::FAdapterUtil::GetBestAdapterForClass(PrimitiveComp->GetClass());

				// Can't be observed
				if(!BestFitAdapter)
				{
					continue;
				}

				// We have an adapter which means it's possible to observe this component so the option to create a manager should be visible
				return true;
			}
		}
	}

	return false;
}

template<typename T>
bool SelectionContains()
{
	static_assert(TIsDerivedFrom<T, AActor>::Value, "Must be an actor type to be a selected actor.");

	USelection* SelectedActors = GEditor->GetSelectedActors();

	TArray<T*> Items;
	SelectedActors->GetSelectedObjects<T>(Items);

	return Items.Num() > 0;
}

bool IsSetAllPlayVisible()
{
	return SelectionContains<AChaosCacheManager>();
}

bool IsSetAllRecordVisible()
{
	return SelectionContains<AChaosCacheManager>();
}

#undef LOCTEXT_NAMESPACE
