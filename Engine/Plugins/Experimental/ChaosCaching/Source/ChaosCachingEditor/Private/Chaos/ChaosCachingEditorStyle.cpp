// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosCachingEditorStyle.h"

#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyleMacros.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleRegistry.h"

FName FChaosCachingEditorStyle::StyleName("ChaosCachingEditorStyle");

FChaosCachingEditorStyle::FChaosCachingEditorStyle()
	: FSlateStyleSet(StyleName)
{
	const FVector2D IconSize(20.0f, 20.0f);
	const FVector2D LabelIconSize(16.0f, 16.0f);

	SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Experimental/ChaosCaching/Content"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	Set("ChaosCachingEditor.Fracture", new IMAGE_BRUSH_SVG("fracture", LabelIconSize));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FChaosCachingEditorStyle::~FChaosCachingEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

FChaosCachingEditorStyle& FChaosCachingEditorStyle::Get()
{
	static FChaosCachingEditorStyle Inst;
	return Inst;
}


