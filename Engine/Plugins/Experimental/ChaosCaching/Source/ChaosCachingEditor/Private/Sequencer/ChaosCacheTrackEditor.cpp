// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/ChaosCacheTrackEditor.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Chaos/CacheManagerActor.h"
#include "Chaos/CacheCollection.h"
#include "LevelSequence.h"
#include "Chaos/Sequencer/MovieSceneChaosCacheTrack.h"
#include "Chaos/Sequencer/MovieSceneChaosCacheSection.h"
#include "Rendering/SlateRenderer.h"
#include "SequencerSectionPainter.h"
#include "ScopedTransaction.h"
#include "MVVM/Views/ViewUtilities.h"
#include "MVVM/ViewModels/ViewDensity.h"
#include "Styling/SlateIconFinder.h"
#include "TimeToPixel.h"
#include "Widgets/SBoxPanel.h"

namespace ChaosCacheEditorConstants
{
	// @todo Sequencer Allow this to be customizable
	constexpr float AnimationTrackHeight = 28.f;
}

#define LOCTEXT_NAMESPACE "FChaosCacheTrackEditor"

static AChaosCacheManager* AcquireChaosCacheFromObjectGuid(const FGuid& Guid, TSharedPtr<ISequencer> SequencerPtr)
{
	UObject* BoundObject = SequencerPtr.IsValid() ? SequencerPtr->FindSpawnedObjectOrTemplate(Guid) : nullptr;

	if (AActor* Actor = Cast<AActor>(BoundObject))
	{
		if (AChaosCacheManager* CacheManager = Cast<AChaosCacheManager>(Actor))
		{
			return CacheManager;
		}
	}

	return nullptr;
}

FChaosCacheSection::FChaosCacheSection(UMovieSceneSection& InSection, TWeakPtr<ISequencer> InSequencer)
	: Section(*CastChecked<UMovieSceneChaosCacheSection>(&InSection))
	, Sequencer(InSequencer)
	, InitialFirstLoopStartOffsetDuringResize(0)
	, InitialStartTimeDuringResize(0)
{ }

UMovieSceneSection* FChaosCacheSection::GetSectionObject()
{
	return &Section;
}

FText FChaosCacheSection::GetSectionTitle() const
{
	if (Section.Params.CacheCollection != nullptr)
	{
		return FText::FromString(Section.Params.CacheCollection->GetName());
	
	}
	return LOCTEXT("NoChaosCacheSection", "No ChaosCache");
}

float FChaosCacheSection::GetSectionHeight(const UE::Sequencer::FViewDensityInfo& ViewDensity) const
{
	return ViewDensity.UniformHeight.Get(ChaosCacheEditorConstants::AnimationTrackHeight);
}

int32 FChaosCacheSection::OnPaintSection(FSequencerSectionPainter& Painter) const
{
	using namespace UE::Sequencer;

	const ESlateDrawEffect DrawEffects = Painter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	const FTimeToPixel& TimeToPixelConverter = Painter.GetTimeConverter();

	int32 LayerId = Painter.PaintSectionBackground();

	static const FSlateBrush* GenericDivider = FAppStyle::GetBrush("Sequencer.GenericDivider");

	if (!Section.HasStartFrame() || !Section.HasEndFrame())
	{
		return LayerId;
	}

	FFrameRate TickResolution = TimeToPixelConverter.GetTickResolution();

	// Add lines where the animation starts and ends/loops
	const float AnimPlayRate = FMath::IsNearlyZero(Section.Params.PlayRate) ? 1.0f : Section.Params.PlayRate;
	const float Duration = Section.Params.GetSequenceLength();
	const float SeqLength = Duration - TickResolution.AsSeconds(Section.Params.StartFrameOffset + Section.Params.EndFrameOffset) / AnimPlayRate;
	const float FirstLoopSeqLength = SeqLength - TickResolution.AsSeconds(Section.Params.FirstLoopStartFrameOffset) / AnimPlayRate;

	if (!FMath::IsNearlyZero(SeqLength, KINDA_SMALL_NUMBER) && SeqLength > 0)
	{
		float MaxOffset = Section.GetRange().Size<FFrameTime>() / TickResolution;
		float OffsetTime = FirstLoopSeqLength;
		float StartTime = Section.GetInclusiveStartFrame() / TickResolution;

		while (OffsetTime < MaxOffset)
		{
			float OffsetPixel = TimeToPixelConverter.SecondsToPixel(StartTime + OffsetTime) - TimeToPixelConverter.SecondsToPixel(StartTime);

			FSlateDrawElement::MakeBox(
				Painter.DrawElements,
				LayerId,
				Painter.SectionGeometry.MakeChild(
					FVector2D(2.f, Painter.SectionGeometry.Size.Y - 2.f),
					FSlateLayoutTransform(FVector2D(OffsetPixel, 1.f))
				).ToPaintGeometry(),
				GenericDivider,
				DrawEffects
			);

			OffsetTime += SeqLength;
		}
	}

	TSharedPtr<ISequencer> SequencerPtr = Sequencer.Pin();
	if (Painter.bIsSelected && SequencerPtr.IsValid())
	{
		FFrameTime CurrentTime = SequencerPtr->GetLocalTime().Time;
		if (Section.GetRange().Contains(CurrentTime.FrameNumber) && Section.Params.CacheCollection != nullptr)
		{
			const float Time = TimeToPixelConverter.FrameToPixel(CurrentTime);

			UChaosCacheCollection* ChaosCache = Section.Params.CacheCollection;

			// Draw the current time next to the scrub handle
			const float AnimTime = Section.MapTimeToAnimation(Duration, CurrentTime, TickResolution);
			int32 FrameTime = FMath::FloorToFloat(TickResolution.AsFrameTime(AnimTime).AsDecimal());
			FString FrameString = FString::FromInt(FrameTime);

			const FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Bold", 10);
			const TSharedRef< FSlateFontMeasure > FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
			FVector2D TextSize = FontMeasureService->Measure(FrameString, SmallLayoutFont);

			// Flip the text position if getting near the end of the view range
			static const float TextOffsetPx = 10.f;
			bool  bDrawLeft = (Painter.SectionGeometry.Size.X - Time) < (TextSize.X + 22.f) - TextOffsetPx;
			float TextPosition = bDrawLeft ? Time - TextSize.X - TextOffsetPx : Time + TextOffsetPx;
			//handle mirrored labels
			const float MajorTickHeight = 9.0f;
			FVector2D TextOffset(TextPosition, Painter.SectionGeometry.Size.Y - (MajorTickHeight + TextSize.Y));

			const FLinearColor DrawColor = FAppStyle::GetSlateColor("SelectionColor").GetColor(FWidgetStyle());
			const FVector2D BoxPadding = FVector2D(4.0f, 2.0f);
			// draw time string

			FSlateDrawElement::MakeBox(
				Painter.DrawElements,
				LayerId + 5,
				Painter.SectionGeometry.ToPaintGeometry(TextSize + 2.0f * BoxPadding, FSlateLayoutTransform(TextOffset - BoxPadding)),
				FAppStyle::GetBrush("WhiteBrush"),
				ESlateDrawEffect::None,
				FLinearColor::Black.CopyWithNewOpacity(0.5f)
			);

			FSlateDrawElement::MakeText(
				Painter.DrawElements,
				LayerId + 6,
				Painter.SectionGeometry.ToPaintGeometry(TextSize, FSlateLayoutTransform(TextOffset)),
				FrameString,
				SmallLayoutFont,
				DrawEffects,
				DrawColor
			);

		}
	}

	return LayerId;
}

void FChaosCacheSection::BeginResizeSection()
{
	InitialFirstLoopStartOffsetDuringResize = Section.Params.FirstLoopStartFrameOffset;
	InitialStartTimeDuringResize = Section.HasStartFrame() ? Section.GetInclusiveStartFrame() : 0;
}

void FChaosCacheSection::UpdateSection(FFrameNumber& UpdateTime) const
{
	const FFrameRate FrameRate = Section.GetTypedOuter<UMovieScene>()->GetTickResolution();
	FFrameNumber StartOffset = FrameRate.AsFrameNumber((UpdateTime - InitialStartTimeDuringResize) / FrameRate * Section.Params.PlayRate);

	StartOffset += InitialFirstLoopStartOffsetDuringResize;

	if (StartOffset < 0)
	{
		// Ensure start offset is not less than 0 and adjust ResizeTime
		UpdateTime = UpdateTime - StartOffset;

		StartOffset = FFrameNumber(0);
	}
	else
	{
		// If the start offset exceeds the length of one loop, trim it back.
		const FFrameNumber SeqLength = FrameRate.AsFrameNumber(Section.Params.GetSequenceLength()) - Section.Params.StartFrameOffset - Section.Params.EndFrameOffset;
		StartOffset = StartOffset % SeqLength;
	}

	Section.Params.FirstLoopStartFrameOffset = StartOffset;
}

void FChaosCacheSection::ResizeSection(ESequencerSectionResizeMode ResizeMode, FFrameNumber ResizeTime)
{
	// Adjust the start offset when resizing from the beginning
	if (ResizeMode == SSRM_LeadingEdge)
	{
		UpdateSection(ResizeTime);
	}

	ISequencerSection::ResizeSection(ResizeMode, ResizeTime);
}

void FChaosCacheSection::BeginSlipSection()
{
	BeginResizeSection();
}

void FChaosCacheSection::SlipSection(FFrameNumber SlipTime)
{
	UpdateSection(SlipTime);

	ISequencerSection::SlipSection(SlipTime);
}

void FChaosCacheSection::BeginDilateSection()
{
	Section.PreviousPlayRate = Section.Params.PlayRate; //make sure to cache the play rate
}
void FChaosCacheSection::DilateSection(const TRange<FFrameNumber>& NewRange, float DilationFactor)
{
	Section.Params.PlayRate = Section.PreviousPlayRate / DilationFactor;
	Section.SetRange(NewRange);
}

FChaosCacheTrackEditor::FChaosCacheTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FMovieSceneTrackEditor(InSequencer)
{ }

TSharedRef<ISequencerTrackEditor> FChaosCacheTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShareable(new FChaosCacheTrackEditor(InSequencer));
}

bool FChaosCacheTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	if (InSequence && InSequence->IsTrackSupported(UMovieSceneChaosCacheTrack::StaticClass()) == ETrackSupport::NotSupported)
	{
		return false;
	}

	return InSequence && InSequence->IsA(ULevelSequence::StaticClass());
}

bool FChaosCacheTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return Type == UMovieSceneChaosCacheTrack::StaticClass();
}

TSharedRef<ISequencerSection> FChaosCacheTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	check(SupportsType(SectionObject.GetOuter()->GetClass()));

	return MakeShareable(new FChaosCacheSection(SectionObject, GetSequencer()));
}

FText FChaosCacheTrackEditor::GetDisplayName() const
{
	return LOCTEXT("ChaosTrackEditor_DisplayName", "Chaos");
}

void FChaosCacheTrackEditor::BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	if (ObjectClass->IsChildOf(AChaosCacheManager::StaticClass()) || ObjectClass->IsChildOf(AActor::StaticClass()))
	{
		if(ObjectBindings.Num() > 0)
		{
			AChaosCacheManager* ChaosCache = AcquireChaosCacheFromObjectGuid(ObjectBindings[0], GetSequencer());

			if (ChaosCache)
			{
				UMovieSceneTrack* Track = nullptr;

				MenuBuilder.AddMenuEntry(
					NSLOCTEXT("Sequencer", "AddChaosCache", "Chaos Cache"),
					NSLOCTEXT("Sequencer", "AddChaosCacheTooltip", "Adds a Chaos Cache track."),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateSP(this, &FChaosCacheTrackEditor::BuildChaosCacheTrack, ObjectBindings, Track)
					)
				);
			}
		}
	}
}

void FChaosCacheTrackEditor::BuildChaosCacheTrack(TArray<FGuid> ObjectBindings, UMovieSceneTrack* Track)
{
	const TSharedPtr<ISequencer> SequencerPtr = GetSequencer();

	if (SequencerPtr.IsValid())
	{
		const FScopedTransaction Transaction(LOCTEXT("AddChaosCache_Transaction", "Add Chaos Cache"));

		for (FGuid ObjectBinding : ObjectBindings)
		{
			if (ObjectBinding.IsValid())
			{
				UObject* Object = SequencerPtr->FindSpawnedObjectOrTemplate(ObjectBinding);

				AChaosCacheManager* ChaosCache = AcquireChaosCacheFromObjectGuid(ObjectBinding, GetSequencer());

				if (Object && ChaosCache)
				{
					AnimatablePropertyChanged(FOnKeyProperty::CreateRaw(this, &FChaosCacheTrackEditor::AddKeyInternal, Object, ChaosCache, Track));
				}
			}
		}
	}
}

FKeyPropertyResult FChaosCacheTrackEditor::AddKeyInternal(FFrameNumber KeyTime, UObject* Object, AChaosCacheManager* ChaosCache, UMovieSceneTrack* Track)
{
	FKeyPropertyResult KeyPropertyResult;

	const FFindOrCreateHandleResult HandleResult = FindOrCreateHandleToObject(Object);
	const FGuid ObjectHandle = HandleResult.Handle;
	KeyPropertyResult.bHandleCreated |= HandleResult.bWasCreated;
	if (ObjectHandle.IsValid())
	{
		if (!Track)
		{
			Track = AddTrack(GetSequencer()->GetFocusedMovieSceneSequence()->GetMovieScene(), ObjectHandle, UMovieSceneChaosCacheTrack::StaticClass(), NAME_None);
			KeyPropertyResult.bTrackCreated = true;
		}

		if (ensure(Track))
		{
			Track->Modify();

			UMovieSceneSection* NewSection = Cast<UMovieSceneChaosCacheTrack>(Track)->AddNewAnimation(KeyTime, ChaosCache);
			KeyPropertyResult.bTrackModified = true;
			KeyPropertyResult.SectionsCreated.Add(NewSection);

			GetSequencer()->EmptySelection();
			GetSequencer()->SelectSection(NewSection);
			GetSequencer()->ThrobSectionSelection();
		}
	}

	return KeyPropertyResult;
}

TSharedPtr<SWidget> FChaosCacheTrackEditor::BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	AChaosCacheManager* ChaosCache = AcquireChaosCacheFromObjectGuid(ObjectBinding, GetSequencer());

	if (ChaosCache)
	{
		TWeakPtr<ISequencer> WeakSequencer = GetSequencer();

		auto SubMenuCallback = [this, ObjectBinding, Track]() -> TSharedRef<SWidget>
		{
			FMenuBuilder MenuBuilder(true, nullptr);

			TArray<FGuid> ObjectBindings;
			ObjectBindings.Add(ObjectBinding);

			BuildChaosCacheTrack(ObjectBindings, Track);
			
			return MenuBuilder.MakeWidget();
		};

		return UE::Sequencer::MakeAddButton(
			LOCTEXT("ChaosCacheText", "Chaos Cache"),
			FOnGetContent::CreateLambda(SubMenuCallback),
			Params.ViewModel);
	}
	else
	{
		return TSharedPtr<SWidget>();
	}

}

const FSlateBrush* FChaosCacheTrackEditor::GetIconBrush() const
{
	return FSlateIconFinder::FindIconForClass(AChaosCacheManager::StaticClass()).GetIcon();
}

#undef LOCTEXT_NAMESPACE
