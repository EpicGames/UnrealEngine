// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/MovieSceneChaosCacheTrackRecorder.h"
#include "Chaos/CacheManagerActor.h"
#include "Chaos/Sequencer/MovieSceneChaosCacheSection.h"
#include "Chaos/Sequencer/MovieSceneChaosCacheTrack.h"
#include "TakeRecorderSettings.h"
#include "Misc/App.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneChaosCacheTrackRecorder)

#define LOCTEXT_NAMESPACE "MovieSceneChaosCacheTrackRecorder"

bool FMovieSceneChaosCacheTrackRecorderFactory::CanRecordObject(UObject* InObjectToRecord) const
{
	return InObjectToRecord->IsA<AChaosCacheManager>();
}

UMovieSceneTrackRecorder* FMovieSceneChaosCacheTrackRecorderFactory::CreateTrackRecorderForObject() const
{
	return NewObject<UMovieSceneChaosCacheTrackRecorder>();
}

void UMovieSceneChaosCacheTrackRecorder::CreateTrackImpl()
{
	ChaosCacheTrack = MovieScene->AddTrack<UMovieSceneChaosCacheTrack>(ObjectGuid);\
	ChaosCacheManager = Cast<AChaosCacheManager>(ObjectToRecord.Get());
	
	ChaosCacheSection = Cast<UMovieSceneChaosCacheSection>(ChaosCacheTrack->CreateNewSection());
	if (ChaosCacheSection.IsValid())
	{
		ChaosCacheSection->SetIsActive(false);
		ChaosCacheTrack->AddSection(*ChaosCacheSection);
		ChaosCacheSection->Params.CacheCollection = ChaosCacheManager->CacheCollection;

		// Resize the section to either it's remaining	keyframes range or 0
		ChaosCacheSection->SetRange(ChaosCacheSection->GetAutoSizeRange().Get(TRange<FFrameNumber>(0, 0)));

		// Make sure it starts at frame 0, in case Auto Size removed a piece of the start
		ChaosCacheSection->ExpandToFrame(0);
	}
}

void UMovieSceneChaosCacheTrackRecorder::SetSectionStartTimecodeImpl(const FTimecode& InSectionStartTimecode, const FFrameNumber& InSectionFirstFrame)
{
	if (ChaosCacheSection.IsValid() && ChaosCacheTrack.IsValid() && ChaosCacheManager)
	{
		ChaosCacheSection->TimecodeSource = FMovieSceneTimecodeSource(InSectionStartTimecode);

		FTakeRecorderParameters Parameters;
		Parameters.User = GetDefault<UTakeRecorderUserSettings>()->Settings;
		Parameters.Project = GetDefault<UTakeRecorderProjectSettings>()->Settings;

		const FFrameRate TickResolution = MovieScene->GetTickResolution();
		const FFrameRate DisplayRate = MovieScene->GetDisplayRate();

		RecordStartTime = FApp::GetCurrentTime();
		RecordStartFrame = Parameters.Project.bStartAtCurrentTimecode ?
			FFrameRate::TransformTime(FFrameTime(InSectionStartTimecode.ToFrameNumber(DisplayRate)),
				DisplayRate, TickResolution).FloorToFrame() : MovieScene->GetPlaybackRange().GetLowerBoundValue();

		StoredCacheMode = ChaosCacheManager->CacheMode;
		ChaosCacheManager->CacheMode = ECacheMode::Record;
		ChaosCacheManager->BeginEvaluate();
	}
}

UMovieSceneSection* UMovieSceneChaosCacheTrackRecorder::GetMovieSceneSection() const
{
	return Cast<UMovieSceneSection>(ChaosCacheSection.Get());
}

void UMovieSceneChaosCacheTrackRecorder::StopRecordingImpl()
{
	if(ChaosCacheManager)
	{
		// Stop the async recorder
		ChaosCacheManager->EndEvaluate();
		ChaosCacheManager->CacheMode = StoredCacheMode;
	}
}

void UMovieSceneChaosCacheTrackRecorder::FinalizeTrackImpl()
{
	if (ChaosCacheSection.IsValid())
	{
		// Set the final range 
		TOptional<TRange<FFrameNumber> > DefaultSectionLength = ChaosCacheSection->GetAutoSizeRange();
		if (DefaultSectionLength.IsSet())
		{
			ChaosCacheSection->SetRange(DefaultSectionLength.GetValue());
		}

		// Activate the section
		ChaosCacheSection->SetIsActive(true);
	}
}

void UMovieSceneChaosCacheTrackRecorder::RecordSampleImpl(const FQualifiedFrameTime& CurrentFrameTime)
{
	// Expand the section to the new length
	const FFrameRate	 TickResolution = MovieScene->GetTickResolution();
	const FFrameNumber CurrentFrame = CurrentFrameTime.ConvertTo(TickResolution).FloorToFrame();

	if (ChaosCacheSection.IsValid())
	{
		ChaosCacheSection->SetEndFrame(CurrentFrame);
	}
}

bool UMovieSceneChaosCacheTrackRecorder::LoadRecordedFile(const FString& FileName, UMovieScene *InMovieScene, TMap<FGuid, AActor*>& ActorGuidToActorMap, TFunction<void()> InCompletionCallback)
{
	return false;
}

#undef LOCTEXT_NAMESPACE
