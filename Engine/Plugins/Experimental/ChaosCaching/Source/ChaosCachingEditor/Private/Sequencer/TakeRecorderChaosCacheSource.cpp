// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/TakeRecorderChaosCacheSource.h"

#include "Chaos/CacheManagerActor.h"
#include "Chaos/Sequencer/MovieSceneChaosCacheTrack.h"
#include "Engine/World.h"
#include "Sequencer/MovieSceneChaosCacheTrackRecorder.h"

#include "LevelSequence.h"
#include "MovieSceneFolder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TakeRecorderChaosCacheSource)


#define LOCTEXT_NAMESPACE "TakeRecorderChaosCacheSource"

UTakeRecorderChaosCacheSource::UTakeRecorderChaosCacheSource(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
	, ChaosCacheManager(nullptr)
{
	TrackTint = FColor(0.0f, 125.0f, 255.0f, 65.0f);
}

TArray<UTakeRecorderSource*> UTakeRecorderChaosCacheSource::PreRecording(ULevelSequence* InSequence, FMovieSceneSequenceID InSequenceID, ULevelSequence* InRootSequence, FManifestSerializer* InManifestSerializer)
{
	if (ChaosCacheManager)
	{
		UMovieScene* MovieScene = InRootSequence->GetMovieScene();
		TrackRecorder = NewObject<UMovieSceneChaosCacheTrackRecorder>();

		// We only support possessable for now since the spawnable template is clearing out the component reference
		// Would be better to use a TSoftObjectPtr for the actor and have a mechanism similar to the FixUpPIE to go
		// from PIE to editor to have a correct soft object path
		const FGuid ObjectGuid = MovieScene->AddPossessable(ChaosCacheManager->GetActorLabel(), ChaosCacheManager->GetClass());
		InRootSequence->BindPossessableObject(ObjectGuid, *ChaosCacheManager, ChaosCacheManager->GetWorld());
		
		TrackRecorder->CreateTrack(nullptr, ChaosCacheManager.Get(), MovieScene, nullptr, ObjectGuid);
		CachedChaosCacheTrack = TrackRecorder->GetChaosCacheTrack();
	}

	return TArray<UTakeRecorderSource*>();
}

void UTakeRecorderChaosCacheSource::TickRecording(const FQualifiedFrameTime& CurrentTime)
{
	if (TrackRecorder)
	{
		TrackRecorder->RecordSample(CurrentTime);
	}
}

void UTakeRecorderChaosCacheSource::StartRecording(const FTimecode& InSectionStartTimecode, const FFrameNumber& InSectionFirstFrame, class ULevelSequence* InSequence)
{
	if (TrackRecorder)
	{
		TrackRecorder->SetSectionStartTimecode(InSectionStartTimecode, InSectionFirstFrame);
	}
}

void UTakeRecorderChaosCacheSource::StopRecording(class ULevelSequence* InSequence)
{
	if (TrackRecorder)
	{
		TrackRecorder->StopRecording();
	}
}

TArray<UTakeRecorderSource*> UTakeRecorderChaosCacheSource::PostRecording(class ULevelSequence* InSequence, ULevelSequence* InRootSequence, const bool bCancelled)
{
	if (TrackRecorder)
	{
		TrackRecorder->FinalizeTrack();
	}

	TrackRecorder = nullptr;
	return TArray<UTakeRecorderSource*>();
}

void UTakeRecorderChaosCacheSource::AddContentsToFolder(UMovieSceneFolder* InFolder)
{
	if (CachedChaosCacheTrack.IsValid())
	{
		InFolder->AddChildTrack(CachedChaosCacheTrack.Get());
	}
}

FText UTakeRecorderChaosCacheSource::GetDisplayTextImpl() const
{
	if (ChaosCacheManager != nullptr && ChaosCacheManager->IsValidLowLevelFast())
	{
		return FText::FromString(ChaosCacheManager->GetName());
	}

	return LOCTEXT("Display Text", "Chaos Cache");
}

FText UTakeRecorderChaosCacheSource::GetAddSourceDisplayTextImpl() const
{
    return LOCTEXT("TakeRecorderDisplayName", "Chaos Recorder");
}

void UTakeRecorderChaosCacheSource::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
}

void UTakeRecorderChaosCacheSource::PostLoad()
{
	Super::PostLoad();
}

#undef LOCTEXT_NAMESPACE
