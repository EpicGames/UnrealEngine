// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorFactories/ActorFactory.h"

#include "ActorFactoryCacheManager.generated.h"

#define UE_API CHAOSCACHINGEDITOR_API

class AActor;
struct FAssetData;

UCLASS(MinimalAPI)
class UActorFactoryCacheManager : public UActorFactory
{
	GENERATED_BODY()

	UE_API UActorFactoryCacheManager();

	//~ Begin UActorFactory Interface
	UE_API virtual bool     CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg) override;
	UE_API virtual void     PostSpawnActor(UObject* Asset, AActor* NewActor) override;
	UE_API virtual UObject* GetAssetFromActorInstance(AActor* ActorInstance) override;
	//~ End UActorFactory Interface
};

#undef UE_API
