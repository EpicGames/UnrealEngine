// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "UObject/WeakObjectPtr.h"

namespace ETextCommit { enum Type : int; }

class IDetailLayoutBuilder;
class IPropertyHandle;
class IDetailChildrenBuilder;

class UObject;
class UChaosCacheCollection;
class UChaosCache;

class SEditableTextBox;

class FCacheCollectionDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:

	UChaosCacheCollection* GetSelectedCollection();
	const UChaosCacheCollection* GetSelectedCollection() const;
	const UChaosCache* GetCacheCollection(int32 Index) const;
	FText GetCacheName(int32 InCacheIndex) const;
	void OnDeleteCache(int32 InArrayIndex, IDetailLayoutBuilder* InLayoutBuilder);

	void OnChangeCacheName(const FText& InNewName, int32 InIndex);
	void OnCommitCacheName(const FText& InNewName, ETextCommit::Type InTextCommit, int32 InIndex);

	void GenerateCacheArrayElementWidget(TSharedRef<IPropertyHandle> PropertyHandle, int32 ArrayIndex, IDetailChildrenBuilder& ChildrenBuilder, IDetailLayoutBuilder* DetailLayout);
	TWeakObjectPtr<UObject> Item;

	TMap< int, TSharedPtr<SEditableTextBox>> NameEditBoxes;
};
