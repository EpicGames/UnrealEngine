// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/AppStyle.h"
#include "Framework/Commands/Commands.h"

class FCachingEditorCommands : public TCommands<FCachingEditorCommands>
{
public:
	FCachingEditorCommands()
		: TCommands<FCachingEditorCommands>(
			TEXT("ChaosCacheEditor"), NSLOCTEXT("ChaosCacheEditorCommands", "ContextDesc", "Chaos Cache"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}

	virtual ~FCachingEditorCommands() = default;

	CHAOSCACHINGEDITOR_API virtual void RegisterCommands();

	TSharedPtr<FUICommandInfo> CreateCacheManager;
};
