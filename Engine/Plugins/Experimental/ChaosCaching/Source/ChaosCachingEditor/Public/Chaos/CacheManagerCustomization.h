// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"

class SWidget;
namespace ESelectInfo { enum Type : int; }

class IDetailLayoutBuilder;
class IPropertyHandle;
class IDetailChildrenBuilder;

class FCacheManagerDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	void SetCacheModeOptions(bool bRecord);
	int32 GetClampedCacheModeIndex() const;
	FSlateColor GetCacheModeTextColor() const;

	void GenerateCacheArrayElementWidget(TSharedRef<IPropertyHandle> PropertyHandle, int32 ArrayIndex, IDetailChildrenBuilder& ChildrenBuilder, IDetailLayoutBuilder* DetailLayout);
	TSharedRef<SWidget> GenerateCacheModeWidget(TSharedPtr<FText> InItem);
	void OnCacheModeChanged(TSharedPtr<FText> NewSelection, ESelectInfo::Type SelectInfo);
	FText GetCacheModeComboBoxContent() const;
	TSharedPtr<FText> GetCurrentCacheMode() const;

	TSharedPtr<IPropertyHandle> CacheModeHandle;
	TArray<TSharedPtr<FText>> CacheModeComboList;
	TArray<FSlateColor> CacheModeColor;
};

class FObservedComponentDetails : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
};
