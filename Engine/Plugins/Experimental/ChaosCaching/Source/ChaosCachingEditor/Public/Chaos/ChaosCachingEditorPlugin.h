// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "Sequencer/MovieSceneChaosCacheTrackRecorder.h"

class FExtender;
class FMenuBuilder;
class FUICommandList;
class IConsoleObject;

class FAssetTypeActions_ChaosCacheCollection;

class UToolMenu;
struct FToolMenuSection;
class AActor;

/**
 * The public interface to this module
 */
class IChaosCachingEditorPlugin : public IModuleInterface
{
	TArray<IConsoleObject*> EditorCommands;

public:
	virtual void StartupModule();
	virtual void ShutdownModule();

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IChaosCachingEditorPlugin& Get()
	{
		return FModuleManager::LoadModuleChecked<IChaosCachingEditorPlugin>("ChaosCachingEditorPlugin");
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable() { return FModuleManager::Get().IsModuleLoaded("ChaosCachingEditorPlugin"); }

private:

	TSharedRef<FExtender> ExtendLevelViewportContextMenu(const TSharedRef<FUICommandList> InCommandList, const TArray<AActor*> SelectedActors);
	void RegisterCachingSubMenu(FMenuBuilder& InMenuBuilder);
	
	void OnCreateCacheManager();

	FDelegateHandle StartupHandle;
	FDelegateHandle TrackEditorBindingHandle;
	FDelegateHandle SourcesMenuExtension;

	FMovieSceneChaosCacheTrackRecorderFactory MovieSceneChaosCacheTrackRecorder;
};
