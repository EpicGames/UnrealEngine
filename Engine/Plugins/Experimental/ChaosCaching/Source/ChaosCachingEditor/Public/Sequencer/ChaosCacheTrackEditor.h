// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencerSection.h"
#include "MovieSceneTrackEditor.h"

struct FBuildEditWidgetParams;

/**
 * Editor for chaos Cache tracks
 */
class FChaosCacheTrackEditor : public FMovieSceneTrackEditor
{
public:
	/**
	 * Constructor
	 *
	 * @param InSequencer The sequencer instance to be used by this tool
	 */
	FChaosCacheTrackEditor( TSharedRef<ISequencer> InSequencer );

	/** Virtual destructor. */
	virtual ~FChaosCacheTrackEditor() = default;

	/**
	 * Creates an instance of this class.  Called by a sequencer 
	 *
	 * @param OwningSequencer The sequencer instance to be used by this tool
	 * @return The new instance of this class
	 */
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor( TSharedRef<ISequencer> OwningSequencer );

public:

	// ~Begin ISequencerTrackEditor interface
	virtual FText GetDisplayName() const override;
	virtual void BuildObjectBindingTrackMenu(class FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> Type) const override;
	virtual TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override;
	virtual const FSlateBrush* GetIconBrush() const override;
	// ~End ISequencerTrackEditor interface
	
private:

	/** Build an internal chaos cache track */
	void BuildChaosCacheTrack(TArray<FGuid> ObjectBindings, UMovieSceneTrack* Track);

	/** Delegate for AnimatablePropertyChanged in AddKey */
	FKeyPropertyResult AddKeyInternal(FFrameNumber KeyTime, UObject* Object, class AChaosCacheManager* ChaosCache, UMovieSceneTrack* Track);
};

/** Chaos cache sequencer section */
class FChaosCacheSection
	: public ISequencerSection
	, public TSharedFromThis<FChaosCacheSection>
{
public:

	/** Constructor. */
	FChaosCacheSection( UMovieSceneSection& InSection, TWeakPtr<ISequencer> InSequencer);

	/** Virtual destructor. */
	virtual ~FChaosCacheSection() = default;

public:

	// ~Begin ISequencerSection interface
	virtual UMovieSceneSection* GetSectionObject() override;
	virtual FText GetSectionTitle() const override;
	virtual float GetSectionHeight(const UE::Sequencer::FViewDensityInfo& ViewDensity) const override;
	virtual int32 OnPaintSection( FSequencerSectionPainter& Painter ) const override;
	virtual void BeginResizeSection() override;
	virtual void ResizeSection(ESequencerSectionResizeMode ResizeMode, FFrameNumber ResizeTime) override;
	virtual void BeginSlipSection() override;
	virtual void SlipSection(FFrameNumber SlipTime) override;
	virtual void BeginDilateSection() override;
	virtual void DilateSection(const TRange<FFrameNumber>& NewRange, float DilationFactor) override;
	// ~End ISequencerSection interface
	
private:

	void UpdateSection(FFrameNumber& UpdateTime) const;

	/** The section we are visualizing */
	class UMovieSceneChaosCacheSection& Section;

	/** Used to draw animation frame, need selection state and local time*/
	TWeakPtr<ISequencer> Sequencer;

	/** Cached first loop start offset value valid only during resize */
	FFrameNumber InitialFirstLoopStartOffsetDuringResize;
	
	/** Cached start time valid only during resize */
	FFrameNumber InitialStartTimeDuringResize;
};
