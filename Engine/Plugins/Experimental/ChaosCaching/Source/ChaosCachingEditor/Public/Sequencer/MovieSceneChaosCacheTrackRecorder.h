// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TrackRecorders/MovieSceneTrackRecorder.h"
#include "TrackRecorders/IMovieSceneTrackRecorderFactory.h"

#include "MovieSceneChaosCacheTrackRecorder.generated.h"

#define UE_API CHAOSCACHINGEDITOR_API

class AChaosCacheManager;
enum class ECacheMode : uint8;

class  UMovieSceneChaosCacheTrack;
class  UMovieSceneChaosCacheSection;

struct FFrameNumber;
class  UMovieSceneSection;
class  UMovieSceneTrackRecorderSettings;

class FMovieSceneChaosCacheTrackRecorderFactory : public IMovieSceneTrackRecorderFactory
{
public:
	virtual ~FMovieSceneChaosCacheTrackRecorderFactory() = default;

	// ~Begin IMovieSceneTrackRecorderFactory Interface
	UE_API virtual bool CanRecordObject(class UObject* InObjectToRecord) const override;
	UE_API virtual UMovieSceneTrackRecorder* CreateTrackRecorderForObject() const override;
	virtual bool CanRecordProperty(class UObject* InObjectToRecord, class FProperty* InPropertyToRecord) const override { return false; }
	virtual UMovieSceneTrackRecorder* CreateTrackRecorderForProperty(UObject* InObjectToRecord, const FName& InPropertyToRecord) const override { return nullptr; }
	virtual FText GetDisplayName() const override { return NSLOCTEXT("MovieSceneChaosCacheTrackRecorderFactory", "DisplayName", "Chaos Cache Track"); }
	// ~End IMovieSceneTrackRecorderFactory Interface
};

/**
* Track recorder implementation for the chaos cache
*/
UCLASS(MinimalAPI, BlueprintType)
class UMovieSceneChaosCacheTrackRecorder
	: public UMovieSceneTrackRecorder
{
	GENERATED_BODY()

public:
	// ~Begin UMovieSceneTrackRecorder Interface
	UE_API virtual void RecordSampleImpl(const FQualifiedFrameTime& CurrentFrameTime) override;
	UE_API virtual void FinalizeTrackImpl() override;
	UE_API virtual void SetSectionStartTimecodeImpl(const FTimecode& InSectionStartTimecode, const FFrameNumber& InSectionFirstFrame) override;
	UE_API virtual UMovieSceneSection* GetMovieSceneSection() const override;
	UE_API virtual void StopRecordingImpl() override;
	UE_API virtual bool LoadRecordedFile(const FString& InFileName, UMovieScene *InMovieScene, TMap<FGuid, AActor*>& ActorGuidToActorMap, TFunction<void()> InCompletionCallback) override;
	UE_API virtual void CreateTrackImpl() override;
	// ~End UMovieSceneTrackRecorder Interface

	/** Returns the chaos cache track on which the cache manager will be recorded */
	TWeakObjectPtr<UMovieSceneChaosCacheTrack> GetChaosCacheTrack() const {return ChaosCacheTrack;}
	
private:
	
	/** The ChaosCache Track to record onto */
	TWeakObjectPtr<UMovieSceneChaosCacheTrack> ChaosCacheTrack;

	/** Sections to record to on each track*/
	TWeakObjectPtr<UMovieSceneChaosCacheSection> ChaosCacheSection;

	/** Chaos cache that will be used to record the simulation */
	AChaosCacheManager* ChaosCacheManager;

	/** Stored cahce mode that will be set back onto the manager once the recording will be finished */
	ECacheMode StoredCacheMode;

	/** The time at the start of this recording section */
	double RecordStartTime;

	/** The frame at the start of this recording section */
	FFrameNumber RecordStartFrame;
};

#undef UE_API
