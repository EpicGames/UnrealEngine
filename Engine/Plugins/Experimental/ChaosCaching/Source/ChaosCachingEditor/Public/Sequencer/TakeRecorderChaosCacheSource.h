// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TakeRecorderSource.h"

#include "TakeRecorderChaosCacheSource.generated.h"

struct FPropertyChangedEvent;

class ULevelSequence;
class UMovieSceneFolder;
class UMovieSceneChaosCacheTrack;
class UMovieSceneChaosCacheTrackRecorder;
class AChaosCacheManager;

/** A recording source selector for the chaos integration into take recorder */

UCLASS(Category = "Chaos")
class UTakeRecorderChaosCacheSource
	: public UTakeRecorderSource
{
public:
	GENERATED_BODY()

	UTakeRecorderChaosCacheSource(const FObjectInitializer& ObjInit);

	/** Chaos Cache manager to be used as take recorder source */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Source", meta = (DisplayName = "Chaos Cache"))
	TSoftObjectPtr<AChaosCacheManager> ChaosCacheManager;
	
	// ~Begin UObject Interface
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostLoad() override;
	// ~End UObject Interface
	
protected:
	// ~Begin UTakeRecorderSource Interface
	virtual bool SupportsSubscenes() const override { return false; }
	// ~End UTakeRecorderSource Interface

private:
	// ~Begin UTakeRecorderSource Interface
	virtual TArray<UTakeRecorderSource*> PreRecording(ULevelSequence* InSequence, FMovieSceneSequenceID InSequenceID, ULevelSequence* InRootSequence, FManifestSerializer* InManifestSerializer) override;
	virtual void StartRecording(const FTimecode& InSectionStartTimecode, const FFrameNumber& InSectionFirstFrame, class ULevelSequence* InSequence) override;
	virtual void StopRecording(class ULevelSequence* InSequence) override;
	virtual void TickRecording(const FQualifiedFrameTime& CurrentTime) override;
	virtual TArray<UTakeRecorderSource*> PostRecording(class ULevelSequence* InSequence, ULevelSequence* InRootSequence, const bool bCancelled) override;
	virtual void AddContentsToFolder(UMovieSceneFolder* InFolder) override;
	virtual FText GetDisplayTextImpl() const override;
	virtual FText GetAddSourceDisplayTextImpl() const override;	
	// ~End UTakeRecorderSource
	
	/** Chaos cache track recorder used by this source */
	UPROPERTY()
	TObjectPtr<UMovieSceneChaosCacheTrackRecorder> TrackRecorder;

	/**
	 * Stores an existing Chaos cache track track in the Sequence to be recorded or
	 * a new one created for recording. Set during PreRecording.
	 */
	TWeakObjectPtr<UMovieSceneChaosCacheTrack> CachedChaosCacheTrack;
};
